// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Common/AvailabilityCheck.h"

#include "HAL/PlatformTime.h"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FAvailabilityCheck::Tick()
{
	if (NextTimestamp != (uint64)-1)
	{
		const uint64 Time = FPlatformTime::Cycles64();
		if (Time > NextTimestamp)
		{
			// Increase wait time with 0.1s, but at no more than 3s.
			WaitTime = FMath::Min(WaitTime + 0.1, 3.0);
			const uint64 WaitTimeCycles64 = static_cast<uint64>(WaitTime / FPlatformTime::GetSecondsPerCycle64());
			NextTimestamp = Time + WaitTimeCycles64;

			return true; // yes, manager can check for (slow) availability conditions
		}
	}

	return false; // no, manager should not check for (slow) availability conditions during this tick
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAvailabilityCheck::Disable()
{
	WaitTime = 0.0;
	NextTimestamp = (uint64)-1;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAvailabilityCheck::Enable(double InWaitTime)
{
	WaitTime = InWaitTime;
	const uint64 WaitTimeCycles64 = static_cast<uint64>(WaitTime / FPlatformTime::GetSecondsPerCycle64());
	NextTimestamp = FPlatformTime::Cycles64() + WaitTimeCycles64;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UEInsights
