// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Common/InsightsCoreStyle.h"

#include "Framework/Application/SlateApplication.h"
#include "HAL/LowLevelMemTracker.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/StyleColors.h"
#include "Styling/ToolBarStyle.h"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FInsightsCoreStyle
////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FInsightsCoreStyle::FStyle> FInsightsCoreStyle::StyleInstance = nullptr;

////////////////////////////////////////////////////////////////////////////////////////////////////

const ISlateStyle& FInsightsCoreStyle::Get()
{
	return *StyleInstance;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsCoreStyle::Initialize()
{
	LLM_SCOPE_BYNAME(TEXT("Insights/Style"));

	// The UE Core style must be initialized before the InsightsCore style
#if WITH_EDITOR
	check(FStarshipCoreStyle::IsInitialized());
#else
	FSlateApplication::InitializeCoreStyle();
#endif

	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FInsightsCoreStyle::FStyle> FInsightsCoreStyle::Create()
{
	TSharedRef<class FInsightsCoreStyle::FStyle> NewStyle = MakeShareable(new FInsightsCoreStyle::FStyle(FInsightsCoreStyle::GetStyleSetName()));
	NewStyle->Initialize();
	return NewStyle;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsCoreStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FName FInsightsCoreStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("InsightsCoreStyle"));
	return StyleSetName;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FInsightsCoreStyle::FStyle
////////////////////////////////////////////////////////////////////////////////////////////////////

FInsightsCoreStyle::FStyle::FStyle(const FName& InStyleSetName)
	: FSlateStyleSet(InStyleSetName)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsCoreStyle::FStyle::SyncParentStyles()
{
	const ISlateStyle* ParentStyle = GetParentStyle();

	NormalText = ParentStyle->GetWidgetStyle<FTextBlockStyle>("NormalText");
	Button = ParentStyle->GetWidgetStyle<FButtonStyle>("Button");

	SelectorColor = ParentStyle->GetSlateColor("SelectorColor");
	SelectionColor = ParentStyle->GetSlateColor("SelectionColor");
	SelectionColor_Inactive = ParentStyle->GetSlateColor("SelectionColor_Inactive");
	SelectionColor_Pressed = ParentStyle->GetSlateColor("SelectionColor_Pressed");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#define EDITOR_IMAGE_BRUSH(RelativePath, ...) IMAGE_BRUSH("../../../Editor/Slate/" RelativePath, __VA_ARGS__)
#define EDITOR_IMAGE_BRUSH_SVG(RelativePath, ...) IMAGE_BRUSH_SVG("../../../Editor/Slate/" RelativePath, __VA_ARGS__)
#define EDITOR_BOX_BRUSH(RelativePath, ...) BOX_BRUSH("../../../Editor/Slate/" RelativePath, __VA_ARGS__)
#define EDITOR_BORDER_BRUSH(RelativePath, ...) BORDER_BRUSH("../../../Editor/Slate/" RelativePath, __VA_ARGS__)
#define TODO_IMAGE_BRUSH(...) EDITOR_IMAGE_BRUSH_SVG("Starship/Common/StaticMesh", __VA_ARGS__)

void FInsightsCoreStyle::FStyle::Initialize()
{
	SetParentStyleName("CoreStyle");

	// Sync styles from the parent style that will be used as templates for styles defined here
	SyncParentStyles();

	Set("Mono.9", DEFAULT_FONT("Mono", 9));
	Set("Mono.10", DEFAULT_FONT("Mono", 10));

	SetContentRoot(FPaths::EngineContentDir() / TEXT("Slate/Starship/Insights"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	const FVector2D Icon12x12(12.0f, 12.0f); // for TreeItem icons
	const FVector2D Icon16x16(16.0f, 16.0f); // for regular icons
	const FVector2D Icon20x20(20.0f, 20.0f); // for ToolBar icons

	//////////////////////////////////////////////////
	// Color brushes

	Set("DarkGreenBrush", new FSlateColorBrush(FLinearColor(0.0f, 0.25f, 0.0f, 1.0f)));

	//////////////////////////////////////////////////
	// Border brushes

	Set("SingleBorder", new FSlateBorderBrush(NAME_None, FMargin(1.0f)));
	Set("DoubleBorder", new FSlateBorderBrush(NAME_None, FMargin(2.0f)));

	Set("EventBorder", new FSlateBorderBrush(NAME_None, FMargin(1.0f)));
	Set("HoveredEventBorder", new FSlateBorderBrush(NAME_None, FMargin(2.0f)));
	Set("SelectedEventBorder", new FSlateBorderBrush(NAME_None, FMargin(2.0f)));

	//////////////////////////////////////////////////
	// Box brushes

	Set("RoundedBackground", new FSlateRoundedBoxBrush(FLinearColor(1.0f, 1.0f, 1.0f, 1.0f), Icon16x16));

	Set("Border.TB", new CORE_BOX_BRUSH("Icons/Profiler/Profiler_Border_TB_16x", FMargin(4.0f / 16.0f)));
	Set("Border.L", new CORE_BOX_BRUSH("Icons/Profiler/Profiler_Border_L_16x", FMargin(4.0f / 16.0f)));
	Set("Border.R", new CORE_BOX_BRUSH("Icons/Profiler/Profiler_Border_R_16x", FMargin(4.0f / 16.0f)));

	//////////////////////////////////////////////////
	// Timing View resources

	//////////////////////////////////////////////////
	// Graph Track resources

	Set("Graph.Point", new EDITOR_IMAGE_BRUSH("Old/Graph/ExecutionBubble", Icon16x16));

	//////////////////////////////////////////////////

	Set("Icons.Debug", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/bug", Icon16x16));
	Set("Icons.Debug.ToolBar", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/bug", Icon20x20));

	Set("Icons.FolderExplore", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/ContentBrowser", Icon16x16));
	//Set("Icons.FolderOpen", new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-open", Icon16x16));		//-> use FAppStyle "Icons.FolderOpen"
	//Set("Icons.FolderClosed", new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-closed", Icon16x16));	//-> use FAppStyle "Icons.FolderClosed"

	Set("Icons.ResetToDefault", new CORE_IMAGE_BRUSH("Icons/Profiler/profiler_ResetToDefault_32x", Icon16x16));
	Set("Icons.DiffersFromDefault", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/ResetToDefault", Icon16x16));

	Set("Icons.TestAutomation", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/TestAutomation", Icon16x16));
	Set("Icons.Test", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Test", Icon16x16));

	Set("Icons.Rename", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Rename", Icon16x16));
	//Set("Icons.Delete", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Delete", Icon16x16));	//-> use FAppStyle "Icons.Delete"

	Set("Icons.Find", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/TraceDataFiltering", Icon16x16));

	//////////////////////////////////////////////////

	Set("Icons.TableTreeView", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/Spreadsheet", Icon16x16));
	Set("Icons.TableTreeView.ToolBar", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/Spreadsheet", Icon20x20));

	Set("Icons.ImportTable", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Import", Icon16x16));

	Set("Icons.Filter.ToolBar", new CORE_IMAGE_BRUSH_SVG("Starship/Common/filter", Icon20x20));
	//Set("Icons.Filter", new CORE_IMAGE_BRUSH_SVG("Starship/Common/filter", Icon16x16));	//-> use FAppStyle "Icons.Filter"
	Set("Icons.FilterAddGroup", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/WorldOutliner", Icon16x16));
	Set("Icons.ClassicFilter", new IMAGE_BRUSH_SVG("Filter", Icon16x16));
	Set("Icons.ClassicFilterConfig", new IMAGE_BRUSH_SVG("FilterConfig", Icon16x16));

	Set("Icons.SortBy", new CORE_IMAGE_BRUSH("Icons/Profiler/profiler_SortBy_32x", Icon16x16));
	//Set("Icons.SortUp", new CORE_IMAGE_BRUSH_SVG("Starship/Common/SortUp", Icon16x16));		//-> use FAppStyle "Icons.SortUp"
	//Set("Icons.SortDown", new CORE_IMAGE_BRUSH_SVG("Starship/Common/SortDown", Icon16x16));	//-> use FAppStyle "Icons.SortDown"

	Set("Icons.ViewColumn", new CORE_IMAGE_BRUSH("Icons/Profiler/profiler_ViewColumn_32x", Icon16x16));
	Set("Icons.ResetColumn", new CORE_IMAGE_BRUSH("Icons/Profiler/profiler_ResetColumn_32x", Icon16x16));

	Set("Icons.ExpandAll", new CORE_IMAGE_BRUSH("Icons/Profiler/profiler_ExpandAll_32x", Icon16x16));
	Set("Icons.CollapseAll", new CORE_IMAGE_BRUSH("Icons/Profiler/profiler_CollapseAll_32x", Icon16x16));
	Set("Icons.ExpandSelection", new CORE_IMAGE_BRUSH("Icons/Profiler/profiler_ExpandSelection_32x", Icon16x16));
	Set("Icons.CollapseSelection", new CORE_IMAGE_BRUSH("Icons/Profiler/profiler_CollapseSelection_32x", Icon16x16));

	//////////////////////////////////////////////////

	Set("TreeTable.RowBackground", new EDITOR_IMAGE_BRUSH("Old/White", Icon16x16, FLinearColor(1.0f, 1.0f, 1.0f, 0.25f)));
	Set("TreeViewBanner.WarningIcon", new CORE_IMAGE_BRUSH_SVG("Starship/Common/alert-circle", Icon20x20, FStyleColors::Warning));

	//////////////////////////////////////////////////
	// Icons for tree/table items

	Set("Icons.Hint.TreeItem", new IMAGE_BRUSH_SVG("InfoTag_12", Icon12x12));
	Set("Icons.Group.TreeItem", new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-closed", Icon12x12));
	Set("Icons.Leaf.TreeItem", new CORE_IMAGE_BRUSH_SVG("Starship/Common/bullet-point", Icon12x12));
	Set("Icons.Asset.TreeItem", new CORE_IMAGE_BRUSH_SVG("Starship/Common/box-perspective", Icon12x12));
	Set("Icons.Package.TreeItem", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/ProjectPackage", Icon12x12));
	Set("Icons.Plugin.TreeItem", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/Plugins", Icon12x12));
	Set("Icons.Dependencies.TreeItem", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/Blueprint", Icon12x12));

	//////////////////////////////////////////////////

	Set("TreeTable.TooltipBold", FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Bold", 8))
		.SetColorAndOpacity(FLinearColor(0.5f, 0.5f, 0.5f, 1.0f))
		.SetShadowOffset(FVector2D(1.0f, 1.0f))
		.SetShadowColorAndOpacity(FLinearColor(0.0f, 0.0f, 0.0f, 0.8f))
	);

	Set("TreeTable.Tooltip", FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Regular", 8))
		.SetColorAndOpacity(FLinearColor::White)
		.SetShadowOffset(FVector2D(1.0f, 1.0f))
		.SetShadowColorAndOpacity(FLinearColor(0.0f, 0.0f, 0.0f, 0.8f))
	);

	Set("TreeTable.NameText", FTextBlockStyle(NormalText)
		.SetColorAndOpacity(FLinearColor::White)
	);

	Set("TreeTable.NormalText", FTextBlockStyle(NormalText)
		.SetColorAndOpacity(FLinearColor::White)
	);

	//////////////////////////////////////////////////

	// NormalEditableTextBox, SearchBox
	// Fixes the vertical alignment of text (inside editable text boxes) to be centered.
	{
		const FEditableTextBoxStyle& NormalEditableTextBoxStyle = FAppStyle::GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox");
		const FEditableTextBoxStyle EditableTextBoxStyle = FEditableTextBoxStyle(NormalEditableTextBoxStyle)
			.SetPadding(FMargin(6.0f, 4.0f, 6.0f, 4.0f));
		Set("NormalEditableTextBox", EditableTextBoxStyle);

		const FSearchBoxStyle& NormalSearchBoxStyle = FAppStyle::GetWidgetStyle<FSearchBoxStyle>("SearchBox");
		FSearchBoxStyle SearchBoxStyle = FSearchBoxStyle(NormalSearchBoxStyle);
		SearchBoxStyle.TextBoxStyle.SetPadding(FMargin(6.0f, 4.0f, 6.0f, 4.0f));
		Set("SearchBox", SearchBoxStyle);
	}

	// PrimaryToolbar
	{
		FToolBarStyle PrimaryToolbarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("SlimToolBar");

		Set("PrimaryToolbar", PrimaryToolbarStyle);

		Set("PrimaryToolbar.MinUniformToolbarSize", 40.0f);
		Set("PrimaryToolbar.MaxUniformToolbarSize", 40.0f);
	}

	// LegacyExtendersToolbarStyle
	{
		FToolBarStyle LegacyExtendersToolbarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("SlimToolBar");
		LegacyExtendersToolbarStyle.BackgroundPadding = FMargin(0.0f);
		Set("LegacyExtendersToolbarStyle", LegacyExtendersToolbarStyle);
	}

	// SecondaryToolbar
	{
		FToolBarStyle SecondaryToolbarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("SlimToolBar");

		SecondaryToolbarStyle.SetBackgroundPadding(         FMargin(4.0f, 4.0f));
		SecondaryToolbarStyle.SetBlockPadding(              FMargin(2.0f, 0.0f));
		SecondaryToolbarStyle.SetButtonPadding(             FMargin(0.0f, 0.0f));
		SecondaryToolbarStyle.SetCheckBoxPadding(           FMargin(2.0f, 0.0f));
		SecondaryToolbarStyle.SetComboButtonPadding(        FMargin(0.0f, 0.0f));
		SecondaryToolbarStyle.SetIndentedBlockPadding(      FMargin(2.0f, 0.0f));
		SecondaryToolbarStyle.SetLabelPadding(              FMargin(2.0f, 0.0f));
		SecondaryToolbarStyle.SetSeparatorPadding(          FMargin(2.0f, -3.0f));

		SecondaryToolbarStyle.ToggleButton.SetPadding(      FMargin(0.0f, 0.0f));

		SecondaryToolbarStyle.ButtonStyle.SetNormalPadding( FMargin(6.0f, 2.0f, 4.0f, 2.0f));
		SecondaryToolbarStyle.ButtonStyle.SetPressedPadding(FMargin(6.0f, 2.0f, 4.0f, 2.0f));

		SecondaryToolbarStyle.WrapButtonStyle.SetWrapButtonPadding(FMargin(0.0f, -2.0f, 0.0f, -2.0f));

		//SecondaryToolbarStyle.IconSize.Set(16.0f, 16.0f);

		Set("SecondaryToolbar", SecondaryToolbarStyle);

		Set("SecondaryToolbar.MinUniformToolbarSize", 32.0f);
		Set("SecondaryToolbar.MaxUniformToolbarSize", 32.0f);
	}

	// SecondaryToolbar2 (used by AutoScroll and NetPacketContentView toolbars)
	{
		FToolBarStyle SecondaryToolbarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("SlimToolBar");

		SecondaryToolbarStyle.SetBackgroundPadding(         FMargin(4.0f, 2.0f));
		SecondaryToolbarStyle.SetBlockPadding(              FMargin(2.0f, 2.0f));
		SecondaryToolbarStyle.SetButtonPadding(             FMargin(0.0f, 2.0f));
		SecondaryToolbarStyle.SetCheckBoxPadding(           FMargin(2.0f, 2.0f));
		SecondaryToolbarStyle.SetComboButtonPadding(        FMargin(0.0f, 2.0f));
		SecondaryToolbarStyle.SetIndentedBlockPadding(      FMargin(2.0f, 2.0f));
		SecondaryToolbarStyle.SetLabelPadding(              FMargin(2.0f, 2.0f));
		SecondaryToolbarStyle.SetSeparatorPadding(          FMargin(2.0f, -1.0f));

		SecondaryToolbarStyle.ToggleButton.SetPadding(      FMargin(0.0f, 0.0f));

		SecondaryToolbarStyle.ButtonStyle.SetNormalPadding( FMargin(3.0f, 0.0f, -1.0f, 0.0f));
		SecondaryToolbarStyle.ButtonStyle.SetPressedPadding(FMargin(3.0f, 0.0f, -1.0f, 0.0f));

		//SecondaryToolbarStyle.IconSize.Set(16.0f, 16.0f);

		Set("SecondaryToolbar2", SecondaryToolbarStyle);

		Set("SecondaryToolbar2.MinUniformToolbarSize", 32.0f);
		Set("SecondaryToolbar2.MaxUniformToolbarSize", 32.0f);
	}

	// Common.GotoNativeCodeHyperlink
	{
		FTextBlockStyle InheritedFromNativeTextStyle = FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 10));

		Set("Common.InheritedFromNativeTextStyle", InheritedFromNativeTextStyle);

		// Go to native class hyperlink
		FButtonStyle EditNativeHyperlinkButton = FButtonStyle()
			.SetNormal(EDITOR_BORDER_BRUSH("Old/HyperlinkDotted", FMargin(0, 0, 0, 3 / 16.0f)))
			.SetPressed(FSlateNoResource())
			.SetHovered(EDITOR_BORDER_BRUSH("Old/HyperlinkUnderline", FMargin(0, 0, 0, 3 / 16.0f)));
		FHyperlinkStyle EditNativeHyperlinkStyle = FHyperlinkStyle()
			.SetUnderlineStyle(EditNativeHyperlinkButton)
			.SetTextStyle(InheritedFromNativeTextStyle)
			.SetPadding(FMargin(0.0f));

		Set("Common.GotoNativeCodeHyperlink", EditNativeHyperlinkStyle);
	}

	//////////////////////////////////////////////////
}

#undef TODO_IMAGE_BRUSH
#undef EDITOR_BOX_BRUSH
#undef EDITOR_IMAGE_BRUSH_SVG
#undef EDITOR_IMAGE_BRUSH

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights
