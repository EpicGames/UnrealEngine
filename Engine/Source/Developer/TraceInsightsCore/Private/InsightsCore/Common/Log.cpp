// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Common/Log.h"

DEFINE_LOG_CATEGORY(LogInsights);

LLM_DEFINE_TAG(Insights);
