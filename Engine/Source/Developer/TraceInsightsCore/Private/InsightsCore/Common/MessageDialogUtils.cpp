// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Common/MessageDialogUtils.h"

#include "Dialog/DialogCommands.h"
#include "Dialog/SCustomDialog.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "UE::Insights::MessageDialogUtils"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

EDialogResponse FMessageDialogUtils::ShowChoiceDialog(const FText& Title, const FText& Content)
{
	if (!FDialogCommands::IsRegistered())
	{
		FDialogCommands::Register();
	}
	TSharedRef<SCustomDialog> Dialog = SNew(SCustomDialog)
		.Title(Title)
		.Content()
		[
			SNew(STextBlock)
			.Text(Content)
		]
		.Buttons({ SCustomDialog::FButton(LOCTEXT("OK", "OK")),
				   SCustomDialog::FButton(LOCTEXT("Cancel", "Cancel")) });

	// returns 0 when OK is pressed, 1 when Cancel is pressed, -1 if the window is closed
	const int32 ButtonPressed = Dialog->ShowModal();
	if (ButtonPressed == 0)
	{
		return EDialogResponse::OK;
	}

	return EDialogResponse::Cancel;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
