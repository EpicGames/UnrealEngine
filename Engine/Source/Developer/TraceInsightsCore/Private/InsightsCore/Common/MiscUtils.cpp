// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Common/MiscUtils.h"

#include "HAL/FileManagerGeneric.h"
#include "HAL/PlatformProcess.h"
#include "Misc/FileHelper.h"
#include "Widgets/Text/STextBlock.h"

// TraceInsightsCore
#include "InsightsCore/Version.h"

#define LOCTEXT_NAMESPACE "UE::Insights::MiscUtils"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMiscUtils::OpenUnrealInsights(const TCHAR* InCmdLine)
{
	if (InCmdLine == nullptr)
	{
		InCmdLine = TEXT("");
	}

	const TCHAR* ExecutablePath = FPlatformProcess::ExecutablePath();

	constexpr bool bLaunchDetached = true;
	constexpr bool bLaunchHidden = false;
	constexpr bool bLaunchReallyHidden = false;

	uint32 ProcessID = 0;
	const int32 PriorityModifier = 0;
	const TCHAR* OptionalWorkingDirectory = nullptr;

	void* PipeWriteChild = nullptr;
	void* PipeReadChild = nullptr;

	FProcHandle Handle = FPlatformProcess::CreateProc(ExecutablePath, InCmdLine, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, &ProcessID, PriorityModifier, OptionalWorkingDirectory, PipeWriteChild, PipeReadChild);
	if (Handle.IsValid())
	{
		FPlatformProcess::CloseProc(Handle);
		return true;
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FVersionWidget::GetWidgetText() const
{
	switch (DisplayMode)
	{
		case EDisplayMode::Version:
			return FText::FromString(UNREAL_INSIGHTS_VERSION_STRING_EX);

		case EDisplayMode::VersionAndMem:
		{
			FPlatformMemoryStats Stats = FPlatformMemory::GetStats();
			FNumberFormattingOptions Options;
			Options.MinimumFractionalDigits = 2;
			Options.MaximumFractionalDigits = 2;
			return FText::Format(LOCTEXT("UnrealInsightsVersionFmt1", "Mem: {0}     {1}"),
				FText::AsMemory(Stats.UsedPhysical, &Options),
				FText::FromString(UNREAL_INSIGHTS_VERSION_STRING_EX));
		}

		case EDisplayMode::VersionAndDetailedMem:
		{
			FPlatformMemoryStats Stats = FPlatformMemory::GetStats();
			FNumberFormattingOptions Options;
			Options.MinimumFractionalDigits = 3;
			Options.MaximumFractionalDigits = 3;
			return FText::Format(LOCTEXT("UnrealInsightsVersionTextFmt2", "Used Physical: {0}     Used Virtual: {1}     {2}"),
				FText::AsMemory(Stats.UsedPhysical, &Options),
				FText::AsMemory(Stats.UsedVirtual, &Options),
				FText::FromString(UNREAL_INSIGHTS_VERSION_STRING_EX));
		}

		default:
			return FText::GetEmpty();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor FVersionWidget::GetColor() const
{
	switch (DisplayMode)
	{
		case EDisplayMode::VersionAndMem:
		case EDisplayMode::VersionAndDetailedMem:
			return FSlateColor(FLinearColor(1.0f, 0.15f, 0.15f, 1.0f));

		case EDisplayMode::Version:
		default:
			return FSlateColor(FLinearColor(0.15f, 0.15f, 0.15f, 1.0f));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply FVersionWidget::OnDoubleClicked(const FGeometry&, const FPointerEvent&)
{
	NextDisplayMode();
	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> FVersionWidget::CreateWidget()
{
	return SNew(STextBlock)
		.Text(this, &FVersionWidget::GetWidgetText)
		.ColorAndOpacity(this, &FVersionWidget::GetColor)
		.OnDoubleClicked(this, &FVersionWidget::OnDoubleClicked);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
