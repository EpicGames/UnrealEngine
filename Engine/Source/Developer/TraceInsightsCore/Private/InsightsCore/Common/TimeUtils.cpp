// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Common/TimeUtils.h"

#include "Misc/StringBuilder.h"

//#include "InsightsCore/Common/Log.h"

#include <cmath>
#include <limits>

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FormatTimeValue(const double Duration, const int32 NumDigits)
{
	if (std::isnan(Duration))
	{
		return TEXT("NaN");
	}

	if (NumDigits <= 0)
	{
		return FString::Printf(TEXT("%.0f"), Duration);
	}

	FString Str = FString::Printf(TEXT("%.*f"), NumDigits, Duration);

	int32 Index = Str.Len() - 1;
	while (Index > 0 && Str[Index] == '0') --Index;
	while (Index > 0 && Str[Index] == '.') --Index;
	++Index;
	if (Index < Str.Len())
	{
		Str.LeftInline(Index, EAllowShrinking::No);
	}

	return Str;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FormatTimeAuto(const double InDuration, const int32 NumDigits)
{
	//TestTimeAutoFormatting();

	double Duration = InDuration;

	if (std::isnan(Duration))
	{
		return TEXT("NaN");
	}

	if (Duration == 0.0)
	{
		return TEXT("0");
	}

	TStringBuilder<64> StrBuilder;

	if (Duration < 0.0)
	{
		Duration = -Duration;
		StrBuilder.AppendChar(TEXT('-'));
	}

	if (Duration == DBL_MAX || Duration == std::numeric_limits<double>::infinity())
	{
		StrBuilder.AppendChar(TEXT('∞'));
	}
	else if (Duration < FTimeValue::Picosecond)
	{
		// (0 .. 1ps)
		return TEXT("~0");
	}
	else if (Duration < FTimeValue::Nanosecond)
	{
		// [1ps .. 1ns)
		if (Duration >= 999.5 * FTimeValue::Picosecond)
		{
			StrBuilder.Append(TEXT("1 ns"));
		}
		else
		{
			const double Picoseconds = Duration * 1000000000000.0 + 0.5;
			const int32 IntPicoseconds = static_cast<int32>(Picoseconds);
			ensure(IntPicoseconds <= 999);
			//if (IntPicoseconds > 999)
			//{
			//	StrBuilder.Append(TEXT("1 ns"));
			//}
			//else
			{
				StrBuilder.Appendf(TEXT("%d ps"), IntPicoseconds);
			}
		}
	}
	else if (Duration < FTimeValue::Microsecond)
	{
		// [1ns .. 1µs)
		if (Duration >= 999.5 * FTimeValue::Nanosecond)
		{
			StrBuilder.Append(TEXT("1 µs"));
		}
		else
		{
			const double Nanoseconds = Duration * 1000000000.0 + 0.5;
			const int32 IntNanoseconds = static_cast<int32>(Nanoseconds);
			ensure(IntNanoseconds <= 999);
			//if (IntNanoseconds > 999)
			//{
			//	StrBuilder.Append(TEXT("1 µs"));
			//}
			//else
			{
				StrBuilder.Appendf(TEXT("%d ns"), IntNanoseconds);
			}
		}
	}
	else if (Duration < FTimeValue::Millisecond)
	{
		// [1µs .. 1ms)
		const double Microseconds = Duration * 1000000.0;
		if (Microseconds >= 999.95)
		{
			StrBuilder.Append(TEXT("1 ms"));
		}
		else
		{
			StrBuilder.Append(FormatTimeValue(Microseconds, NumDigits));
			StrBuilder.Append(TEXT(" µs"));
		}
	}
	else if (Duration < FTimeValue::Second)
	{
		// [1ms .. 1s)
		const double Milliseconds = Duration * 1000.0;
		if (Milliseconds >= 999.95)
		{
			StrBuilder.Append(TEXT("1s"));
		}
		else
		{
			StrBuilder.Append(FormatTimeValue(Milliseconds, NumDigits));
			StrBuilder.Append(TEXT(" ms"));
		}
	}
	else if (Duration < FTimeValue::Minute)
	{
		// [1s .. 1m)
		if (Duration >= 59.95)
		{
			StrBuilder.Append(TEXT("1m"));
		}
		else
		{
			StrBuilder.Append(FormatTimeValue(Duration, NumDigits));
			StrBuilder.AppendChar(TEXT('s'));
		}
	}
	else if (Duration < FTimeValue::Hour)
	{
		// [1m .. 1h)
		const double Minutes = FMath::FloorToDouble(Duration / FTimeValue::Minute);
		StrBuilder.Appendf(TEXT("%dm"), static_cast<int32>(Minutes));
		Duration -= Minutes * FTimeValue::Minute;
		if (NumDigits <= 1)
		{
			const double Seconds = FMath::FloorToDouble(Duration / FTimeValue::Second);
			if (Seconds > 0.5)
			{
				StrBuilder.Appendf(TEXT(" %ds"), static_cast<int32>(Seconds));
			}
		}
		else
		{
			StrBuilder.AppendChar(TEXT(' '));
			StrBuilder.Append(FormatTimeValue(Duration, NumDigits - 1));
			StrBuilder.AppendChar(TEXT('s'));
		}
	}
	else if (Duration < FTimeValue::Day)
	{
		// [1h .. 1d)
		const double Hours = FMath::FloorToDouble(Duration / FTimeValue::Hour);
		StrBuilder.Appendf(TEXT("%dh"), static_cast<int32>(Hours));
		Duration -= Hours * FTimeValue::Hour;
		const double Minutes = FMath::FloorToDouble(Duration / FTimeValue::Minute);
		if (Minutes > 0.5)
		{
			StrBuilder.Appendf(TEXT(" %dm"), static_cast<int32>(Minutes));
		}
	}
	else
	{
		// [1d .. ∞)
		const double Days = FMath::FloorToDouble(Duration / FTimeValue::Day);
		StrBuilder.Appendf(TEXT("%dd"), static_cast<int32>(Days));
		Duration -= Days * FTimeValue::Day;
		const double Hours = FMath::FloorToDouble(Duration / FTimeValue::Hour);
		if (Hours > 0.5)
		{
			StrBuilder.Appendf(TEXT(" %dh"), static_cast<int32>(Hours));
		}
	}

	return FString(StrBuilder);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FormatTimeMs(const double InDuration, const int32 NumDigits, bool bAddTimeUnit)
{
	if (std::isnan(InDuration))
	{
		return TEXT("NaN");
	}

	if (FMath::IsNearlyZero(InDuration, FTimeValue::Picosecond))
	{
		return TEXT("0");
	}

	double Duration = InDuration;
	TStringBuilder<64> StrBuilder;

	if (Duration < 0.0)
	{
		Duration = -Duration;
		StrBuilder.AppendChar(TEXT('-'));
	}

	if (Duration == DBL_MAX || Duration == std::numeric_limits<double>::infinity())
	{
		StrBuilder.AppendChar(TEXT('∞'));
	}
	else
	{
		if (NumDigits <= 0)
		{
			StrBuilder.Appendf(TEXT("%.0f"), Duration * 1000.0);
		}
		else
		{
#if !PLATFORM_USE_GENERIC_STRING_IMPLEMENTATION
			StrBuilder.Appendf(TEXT("%.*f"), NumDigits, Duration * 1000.0);
#else
			// proper resolution is tracked as UE-79534
			TCHAR FormatString[32];
			FCString::Snprintf(FormatString, sizeof(FormatString), TEXT("%%.%df"), NumDigits);
			StrBuilder.Appendf(FormatString, Duration * 1000.0);
#endif
		}

		if (bAddTimeUnit)
		{
			StrBuilder.Append(TEXT(" ms"));
		}
	}

	return FString(StrBuilder);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FormatTime(const double InTime, const double Precision)
{
	if (std::isnan(InTime))
	{
		return TEXT("NaN");
	}

	if (FMath::IsNearlyZero(InTime, FMath::Max(FTimeValue::Picosecond, Precision / 10.0f)))
	{
		return TEXT("0");
	}

	double Time = InTime;
	TStringBuilder<64> StrBuilder;

	if (Time < 0.0)
	{
		Time = -Time;
		StrBuilder.AppendChar(TEXT('-'));
	}

	if (Time == DBL_MAX || Time == std::numeric_limits<double>::infinity())
	{
		StrBuilder.AppendChar(TEXT('∞'));
		return FString(StrBuilder);
	}

	bool bIsSpaceNeeded = false;

	int32 Days = static_cast<int32>(Time / FTimeValue::Day);
	if (Days > 0)
	{
		StrBuilder.Appendf(TEXT("%dd"), Days);
		bIsSpaceNeeded = true;
		Time -= static_cast<double>(Days) * FTimeValue::Day;
	}

	if (Precision >= FTimeValue::Day)
	{
		if (!bIsSpaceNeeded)
		{
			return TEXT("~0");
		}
		return FString(StrBuilder);
	}

	int32 Hours = static_cast<int32>(Time / FTimeValue::Hour);
	if (Hours > 0)
	{
		if (bIsSpaceNeeded)
		{
			StrBuilder.AppendChar(TEXT(' '));
		}
		StrBuilder.Appendf(TEXT("%dh"), Hours);
		bIsSpaceNeeded = true;
		Time -= static_cast<double>(Hours) * FTimeValue::Hour;
	}

	if (Precision >= FTimeValue::Hour)
	{
		if (!bIsSpaceNeeded)
		{
			return TEXT("~0");
		}
		return FString(StrBuilder);
	}

	int32 Minutes = static_cast<int32>(Time / FTimeValue::Minute);
	if (Minutes > 0)
	{
		if (bIsSpaceNeeded)
		{
			StrBuilder.AppendChar(TEXT(' '));
		}
		StrBuilder.Appendf(TEXT("%dm"), Minutes);
		bIsSpaceNeeded = true;
		Time -= static_cast<double>(Minutes) * FTimeValue::Minute;
	}

	if (Precision >= FTimeValue::Minute)
	{
		if (!bIsSpaceNeeded)
		{
			return TEXT("~0");
		}
		return FString(StrBuilder);
	}

	//TestOptimizationIssue();

	////float Log10 = -FMath::LogX(10.0f, static_cast<float>(Precision));
	//double Log10 = -log10(Precision);
	//int32 Digits = (Log10 > 0) ? FMath::CeilToInt(Log10) : 0;

	static const double DigitThresholds[] =
	{
		FTimeValue::Second,              // 0 digits
		FTimeValue::Millisecond * 100,   // 1 digit
		FTimeValue::Millisecond * 10,    // 2 digits
		FTimeValue::Millisecond,         // 3 digits
		FTimeValue::Microsecond * 100,   // 4 digits
		FTimeValue::Microsecond * 10,    // 5 digits
		FTimeValue::Microsecond,         // 6 digits
		FTimeValue::Nanosecond * 100,    // 7 digits
		FTimeValue::Nanosecond * 10,     // 8 digits
		FTimeValue::Nanosecond,          // 9 digits
		FTimeValue::Picosecond * 100,    // 10 digits
		FTimeValue::Picosecond * 10,     // 11 digits
		FTimeValue::Picosecond,          // 12 digits
		0                               // 13 digits
	};
	int32 Digits = 0;
	while (Precision < DigitThresholds[Digits])
	{
		Digits++;
	}

	if (Digits == 0)
	{
		int32 Seconds = static_cast<int32>(Time / FTimeValue::Second + 0.5);
		if (Seconds > 0)
		{
			if (bIsSpaceNeeded)
			{
				StrBuilder.AppendChar(TEXT(' '));
			}
			StrBuilder.Appendf(TEXT("%ds"), Seconds);
		}
		else if (!bIsSpaceNeeded)
		{
			return TEXT("~0");
		}
	}
	//else if (Digits <= 9)
	//{
	//	if (bIsSpaceNeeded)
	//	{
	//		StrBuilder.AppendChar(TEXT(' '));
	//	}
	//	int32 Seconds = static_cast<int32>(Time / FTimeValue::Second);
	//	Time -= static_cast<double>(Seconds) * FTimeValue::Second;
	//	int64 SubSeconds = static_cast<int64>(Time * FMath::Pow(10.0f, Digits) + 0.5);
	//	StrBuilder.Appendf(TEXT("%d.%0*llds"), Seconds, Digits, SubSeconds);
	//}
	else
	{
		if (bIsSpaceNeeded)
		{
			StrBuilder.AppendChar(TEXT(' '));
		}
#if !PLATFORM_USE_GENERIC_STRING_IMPLEMENTATION
		StrBuilder.Appendf(TEXT("%.*fs"), Digits, Time);
#else
		// proper resolution is tracked as UE-79534
		TCHAR FormatString[32];
		FCString::Snprintf(FormatString, sizeof(FormatString), TEXT("%%.%dfs"), Digits);
		StrBuilder.Appendf(FormatString, Time);
#endif
	}

	return FString(StrBuilder);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FormatTimeHMS(const double Time, const double Precision)
{
	//TODO: DD:HH:MM:SS.mmm.uuu.nnn.ppp
	return FormatTime(Time, Precision);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SplitTime(const double InTime, FTimeSplit& OutTimeSplit)
{
	double Time = InTime;

	OutTimeSplit.bIsZero = false;
	OutTimeSplit.bIsNegative = false;
	OutTimeSplit.bIsInfinite = false;
	OutTimeSplit.bIsNaN = false;

	if (std::isnan(InTime))
	{
		OutTimeSplit.bIsNaN = true;
		return;
	}

	if (FMath::IsNearlyZero(Time, FTimeValue::Picosecond))
	{
		OutTimeSplit.bIsZero = true;
		for (int32 Index = 0; Index < 8; ++Index)
		{
			OutTimeSplit.Units[Index] = 0;
		}
		return;
	}

	if (Time < 0.0)
	{
		Time = -Time;
		OutTimeSplit.bIsNegative = true;
	}

	if (Time == DBL_MAX || Time == std::numeric_limits<double>::infinity())
	{
		OutTimeSplit.bIsInfinite = true;
		for (int32 Index = 0; Index < 8; ++Index)
		{
			OutTimeSplit.Units[Index] = 0;
		}
		return;
	}

	bool bIsZero = true; // assume true and see if any split unit is != 0

	const double Days = FMath::FloorToDouble(Time / FTimeValue::Day);
	OutTimeSplit.Days = static_cast<int32>(Days);
	if (OutTimeSplit.Days > 0)
	{
		Time -= Days * FTimeValue::Day;
		bIsZero = false;
	}

	const double Hours = FMath::FloorToDouble(Time / FTimeValue::Hour);
	OutTimeSplit.Hours = static_cast<int32>(Hours);
	if (OutTimeSplit.Hours > 0)
	{
		Time -= Hours * FTimeValue::Hour;
	}

	const double Minutes = FMath::FloorToDouble(Time / FTimeValue::Minute);
	OutTimeSplit.Minutes = static_cast<int32>(Minutes);
	if (OutTimeSplit.Minutes > 0)
	{
		Time -= Minutes * FTimeValue::Minute;
		bIsZero = false;
	}

	const double Seconds = FMath::FloorToDouble(Time / FTimeValue::Second);
	OutTimeSplit.Seconds = static_cast<int32>(Seconds);
	if (OutTimeSplit.Seconds > 0)
	{
		Time -= Seconds * FTimeValue::Second;
		bIsZero = false;
	}

	const double Milliseconds = FMath::FloorToDouble(Time / FTimeValue::Millisecond);
	OutTimeSplit.Milliseconds = static_cast<int32>(Milliseconds);
	if (OutTimeSplit.Milliseconds > 0)
	{
		Time -= Milliseconds * FTimeValue::Millisecond;
		bIsZero = false;
	}

	const double Microseconds = FMath::FloorToDouble(Time / FTimeValue::Microsecond);
	OutTimeSplit.Microseconds = static_cast<int32>(Microseconds);
	if (OutTimeSplit.Microseconds > 0)
	{
		Time -= Microseconds * FTimeValue::Microsecond;
		bIsZero = false;
	}

	const double Nanoseconds = FMath::FloorToDouble(Time / FTimeValue::Nanosecond);
	OutTimeSplit.Nanoseconds = static_cast<int32>(Nanoseconds);
	if (OutTimeSplit.Nanoseconds > 0)
	{
		Time -= Nanoseconds * FTimeValue::Nanosecond;
		bIsZero = false;
	}

	const double Picoseconds = FMath::FloorToDouble(Time / FTimeValue::Picosecond);
	OutTimeSplit.Picoseconds = static_cast<int32>(Picoseconds);
	if (OutTimeSplit.Picoseconds > 0)
	{
		Time -= Picoseconds * FTimeValue::Picosecond;
		bIsZero = false;
	}

	OutTimeSplit.bIsZero = bIsZero;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FormatTimeSplit(const FTimeSplit& InTimeSplit, const double Precision)
{
	if (InTimeSplit.bIsNaN)
	{
		return TEXT("NaN");
	}

	if (InTimeSplit.bIsZero)
	{
		return TEXT("0");
	}

	if (InTimeSplit.bIsInfinite)
	{
		return InTimeSplit.bIsNegative ? TEXT("-∞") : TEXT("∞");
	}

	TStringBuilder<64> StrBuilder;

	if (InTimeSplit.bIsNegative)
	{
		StrBuilder.AppendChar(TEXT('-'));
	}

	bool bIsSpaceNeeded = false;

	if (InTimeSplit.Days > 0)
	{
		StrBuilder.Appendf(TEXT("%dd"), InTimeSplit.Days);
		bIsSpaceNeeded = true;
	}

	if (Precision >= FTimeValue::Day)
	{
		if (!bIsSpaceNeeded)
		{
			return TEXT("~0");
		}
		return FString(StrBuilder);
	}

	if (InTimeSplit.Hours > 0)
	{
		if (bIsSpaceNeeded)
		{
			StrBuilder.AppendChar(TEXT(' '));
		}
		StrBuilder.Appendf(TEXT("%dh"), InTimeSplit.Hours);
		bIsSpaceNeeded = true;
	}

	if (Precision >= FTimeValue::Hour)
	{
		if (!bIsSpaceNeeded)
		{
			return TEXT("~0");
		}
		return FString(StrBuilder);
	}

	if (InTimeSplit.Minutes > 0)
	{
		if (bIsSpaceNeeded)
		{
			StrBuilder.AppendChar(TEXT(' '));
		}
		StrBuilder.Appendf(TEXT("%dm"), InTimeSplit.Minutes);
		bIsSpaceNeeded = true;
	}

	if (Precision >= FTimeValue::Minute)
	{
		if (!bIsSpaceNeeded)
		{
			return TEXT("~0");
		}
		return FString(StrBuilder);
	}

	if (InTimeSplit.Seconds > 0)
	{
		if (bIsSpaceNeeded)
		{
			StrBuilder.AppendChar(TEXT(' '));
		}
		StrBuilder.Appendf(TEXT("%ds"), InTimeSplit.Seconds);
		bIsSpaceNeeded = true;
	}

	if (Precision >= FTimeValue::Second)
	{
		if (!bIsSpaceNeeded)
		{
			return TEXT("~0");
		}
		return FString(StrBuilder);
	}

	if (InTimeSplit.Milliseconds > 0)
	{
		if (bIsSpaceNeeded)
		{
			StrBuilder.AppendChar(TEXT(' '));
		}
		StrBuilder.Appendf(TEXT("%dms"), InTimeSplit.Milliseconds);
		bIsSpaceNeeded = true;
	}

	if (Precision >= FTimeValue::Millisecond)
	{
		if (!bIsSpaceNeeded)
		{
			return TEXT("~0");
		}
		return FString(StrBuilder);
	}

	if (InTimeSplit.Microseconds > 0)
	{
		if (bIsSpaceNeeded)
		{
			StrBuilder.AppendChar(TEXT(' '));
		}
		StrBuilder.Appendf(TEXT("%dµs"), InTimeSplit.Microseconds);
		bIsSpaceNeeded = true;
	}

	if (Precision >= FTimeValue::Microsecond)
	{
		if (!bIsSpaceNeeded)
		{
			return TEXT("~0");
		}
		return FString(StrBuilder);
	}

	if (InTimeSplit.Nanoseconds > 0)
	{
		if (bIsSpaceNeeded)
		{
			StrBuilder.AppendChar(TEXT(' '));
		}
		StrBuilder.Appendf(TEXT("%dns"), InTimeSplit.Nanoseconds);
		bIsSpaceNeeded = true;
	}

	if (Precision >= FTimeValue::Nanosecond)
	{
		if (!bIsSpaceNeeded)
		{
			return TEXT("~0");
		}
		return FString(StrBuilder);
	}

	if (InTimeSplit.Picoseconds > 0)
	{
		if (bIsSpaceNeeded)
		{
			StrBuilder.AppendChar(TEXT(' '));
		}
		StrBuilder.Appendf(TEXT("%dps"), InTimeSplit.Picoseconds);
		bIsSpaceNeeded = true;
	}

	if (!bIsSpaceNeeded)
	{
		return TEXT("~0");
	}

	return FString(StrBuilder);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FormatTimeSplit(const double Time, const double Precision)
{
	FTimeSplit TimeSplit;
	SplitTime(Time, TimeSplit);
	return FormatTimeSplit(TimeSplit, Precision);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void TestTimeFormatting()
{
	double T1 = 1 * FTimeValue::Day + 2 * FTimeValue::Hour + 3 * FTimeValue::Minute + 4.567890123456789;
	FString S1 = FormatTime(T1, FTimeValue::Day);
	//UE_LOGF(LogInsights, Log, "D-T: %ls", *S1);
	FString S2 = FormatTime(T1, FTimeValue::Hour);
	//UE_LOGF(LogInsights, Log, "H-T: %ls", *S2);
	FString S3 = FormatTime(T1, FTimeValue::Minute);
	//UE_LOGF(LogInsights, Log, "M-T: %ls", *S3);
	for (double P = 10.0; P >= FTimeValue::Picosecond; P /= 10.0)
	{
		FString SP = FormatTime(T1, P);
		//UE_LOGF(LogInsights, Log, "P:%g T: %ls", P, *SP);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void TestTimeAutoFormatting()
{
	static bool Once = false;
	if (Once) return;
	Once = true;

	struct FTestTimeAutoData
	{
		double T;
		const TCHAR* Msg;
	};

	static const FTestTimeAutoData Data[] =
	{
		{ FTimeValue::Minute,                TEXT("1m") },
		{ FTimeValue::Second * 59.99,        TEXT("59.99s") },
		{ FTimeValue::Second * 59.95,        TEXT("59.95s") },
		{ 0, TEXT("[threshold 1m / 59.9s]") },
		{ FTimeValue::Second * 59.94,        TEXT("59.94s") },
		{ FTimeValue::Second * 59.9,         TEXT("59.9s") },
		{ FTimeValue::Second * 10,           TEXT("10s") },
		{ FTimeValue::Second,                TEXT("1s") },
		{ FTimeValue::Millisecond * 999.99,  TEXT("999.99ms") },
		{ FTimeValue::Millisecond * 999.95,  TEXT("999.95ms") },
		{ 0, TEXT("[threshold 1s / 999.9 ms]") },
		{ FTimeValue::Millisecond * 999.94,  TEXT("999.94ms") },
		{ FTimeValue::Millisecond * 999.9,   TEXT("999.9ms") },
		{ FTimeValue::Millisecond * 999,     TEXT("999ms") },
		{ FTimeValue::Millisecond * 100,     TEXT("100ms") },
		{ FTimeValue::Millisecond * 10,      TEXT("10ms") },
		{ FTimeValue::Millisecond * 1.55,    TEXT("1.55ms") },
		{ FTimeValue::Millisecond * 1.5,     TEXT("1.5ms") },
		{ FTimeValue::Millisecond * 1.05,    TEXT("1.05ms") },
		{ FTimeValue::Millisecond,           TEXT("1ms") },
		{ FTimeValue::Microsecond * 999.99,  TEXT("999.99µs") },
		{ FTimeValue::Microsecond * 999.95,  TEXT("999.95µs") },
		{ 0, TEXT("[threshold 1 ms / 999.9 µs]") },
		{ FTimeValue::Microsecond * 999.94,  TEXT("999.94µs") },
		{ FTimeValue::Microsecond * 999.9,   TEXT("999.9µs") },
		{ FTimeValue::Microsecond * 999,     TEXT("999µs") },
		{ FTimeValue::Microsecond * 100,     TEXT("100µs") },
		{ FTimeValue::Microsecond * 10,      TEXT("10µs") },
		{ FTimeValue::Microsecond,           TEXT("1µs") },
		{ FTimeValue::Nanosecond * 999.9,    TEXT("999.9ns") },
		{ FTimeValue::Nanosecond * 999.5,    TEXT("999.5ns") },
		{ 0, TEXT("[threshold 1 µs / 999 ns]") },
		{ FTimeValue::Nanosecond * 999.4,    TEXT("999.4ns") },
		{ FTimeValue::Nanosecond * 999,      TEXT("999ns") },
		{ FTimeValue::Nanosecond * 100,      TEXT("100ns") },
		{ FTimeValue::Nanosecond * 10,       TEXT("10ns") },
		{ FTimeValue::Nanosecond,            TEXT("1ns") },
		{ FTimeValue::Picosecond * 999.9,    TEXT("999.9ps") },
		{ FTimeValue::Picosecond * 999.5,    TEXT("999.5ps") },
		{ 0, TEXT("[threshold 1 ns / 999 ps]") },
		{ FTimeValue::Picosecond * 999.4,    TEXT("999.4ps") },
		{ FTimeValue::Picosecond * 999,      TEXT("999ps") },
		{ FTimeValue::Picosecond * 100,      TEXT("100ps") },
		{ FTimeValue::Picosecond * 10,       TEXT("10ps") },
		{ FTimeValue::Picosecond,            TEXT("1ps") },
		{ FTimeValue::Picosecond * 0.1,      TEXT("0.1ps") },
	};
	const int32 DataCount = sizeof(Data) / sizeof(FTestTimeAutoData);

	for (int32 Index = 0; Index < DataCount; ++Index)
	{
		FString Str = FormatTimeAuto(Data[Index].T);
		//UE_LOGF(LogInsights, Log, "%ls : %ls", Data[Index].Msg, *Str);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 GetNumDigits(const double Precision)
{
	//float Log10 = -log10(Precision);
	float Log10 = -FMath::LogX(10.0f, static_cast<float>(Precision));
	int32 D = (Log10 > 0) ? FMath::CeilToInt(Log10) : 0;
	return D;
}

UE_DISABLE_OPTIMIZATION_SHIP
int32 GetNumDigitsOptDisabled(const double Precision)
{
	//double Log10 = -log10(Precision);
	float Log10 = -FMath::LogX(10.0f, static_cast<float>(Precision));
	int32 D = (Log10 > 0) ? FMath::CeilToInt(Log10) : 0;
	return D;
}
UE_ENABLE_OPTIMIZATION_SHIP

UE_DISABLE_OPTIMIZATION_SHIP
void TestOptimizationIssue()
{
	constexpr double Ns = 0.000000001;
	int32 D1 = GetNumDigits(Ns);
	int32 D2 = GetNumDigitsOptDisabled(Ns);
	//UE_LOGF(LogInsights, Log, "D1 = %d, D2 = %d", D1, D2);
	ensure(D1 == 9); // 10 ?
	ensure(D1 == D2);
}
UE_ENABLE_OPTIMIZATION_SHIP

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights
