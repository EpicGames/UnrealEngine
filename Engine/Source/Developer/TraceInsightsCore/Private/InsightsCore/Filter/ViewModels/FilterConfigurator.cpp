// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Filter/ViewModels/FilterConfigurator.h"

#include "InsightsCore/Filter/Widgets/SFilterConfigurator.h"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

FFilterConfigurator::FFilterConfigurator()
{
	RootNode = MakeShared<FFilterConfiguratorNode>(true);
	AvailableFilters = MakeShared<TArray<TSharedPtr<FFilter>>>();
	RootNode->SetAvailableFilters(AvailableFilters);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FFilterConfigurator::~FFilterConfigurator()
{
	OnDestroyedEvent.Broadcast();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FFilterConfigurator::FFilterConfigurator(const FFilterConfigurator& Other)
{
	*this = Other;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FFilterConfigurator& FFilterConfigurator::operator=(const FFilterConfigurator& Other)
{
	if (*this == Other)
	{
		return *this;
	}

	RootNode = FFilterConfiguratorNode::DeepCopy(*Other.RootNode);

	AvailableFilters = Other.AvailableFilters;
	OnDestroyedEvent = Other.OnDestroyedEvent;

	ComputeUsedKeys();
	RootNode->ProcessFilter();

	OnChangesCommittedEvent.Broadcast();

	return *this;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FFilterConfigurator::operator==(const FFilterConfigurator& Other) const
{
	bool bIsEqual = AvailableFilters.Get() == Other.AvailableFilters.Get();
	bIsEqual &= RootNode.IsValid() == Other.RootNode.IsValid();
	if (RootNode.IsValid() && Other.RootNode.IsValid())
	{
		bIsEqual &= *RootNode == *Other.RootNode;
	}

	return bIsEqual;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FFilterConfigurator::ApplyFilters(const FBaseFilterContext& Context) const
{
	return RootNode->ApplyFilters(Context);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FFilterConfigurator::IsKeyUsed(int32 Key) const
{
	return KeysUsed.Contains(Key);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFilterConfigurator::ComputeUsedKeys()
{
	KeysUsed.Reset();
	RootNode->GetUsedKeys(KeysUsed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights
