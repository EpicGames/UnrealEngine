// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Filter/ViewModels/FilterConfiguratorNode.h"

#include "Misc/Parse.h"

#include "InsightsCore/Filter/ViewModels/Filters.h"

#define LOCTEXT_NAMESPACE "UE::Insights::FFilterConfiguratorNode"

namespace UE::Insights
{

INSIGHTS_IMPLEMENT_RTTI(FFilterConfiguratorNode)

////////////////////////////////////////////////////////////////////////////////////////////////////

FFilterConfiguratorNode::FFilterConfiguratorNode(bool bInIsGroup)
	: FBaseTreeNode(bInIsGroup)
{
	if (bInIsGroup)
	{
		SelectedFilterGroupOperator = GetFilterGroupOperators()[0];
	}

	AvailableFilterOperators = MakeShared<TArray<TSharedPtr<IFilterOperator>>>();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FFilterConfiguratorNode> FFilterConfiguratorNode::DeepCopy(const FFilterConfiguratorNode& InSourceNode)
{
	TSharedRef<FFilterConfiguratorNode> NodeCopyPtr = MakeShared<FFilterConfiguratorNode>(InSourceNode.IsGroup());
	FFilterConfiguratorNode& NodeCopy = *NodeCopyPtr;

	NodeCopy.AvailableFilters = InSourceNode.AvailableFilters;
	NodeCopy.AvailableFilterOperators = InSourceNode.AvailableFilterOperators;
	NodeCopy.SelectedFilter = InSourceNode.SelectedFilter;
	NodeCopy.SelectedFilterGroupOperator = InSourceNode.SelectedFilterGroupOperator;
	NodeCopy.TextBoxValue = InSourceNode.TextBoxValue;

	NodeCopy.SetExpansion(InSourceNode.IsExpanded());

	if (InSourceNode.IsGroup())
	{
		check(NodeCopy.IsGroup());
		NodeCopy.ClearChildren(InSourceNode.GetChildrenCount());
		for (FBaseTreeNodePtr Child : InSourceNode.GetChildren())
		{
			check(Child.IsValid() && Child->Is<FFilterConfiguratorNode>());
			TSharedPtr<FFilterConfiguratorNode> ChildCopy = FFilterConfiguratorNode::DeepCopy(Child->As<FFilterConfiguratorNode>());
			NodeCopy.AddChildAndSetParent(ChildCopy);
		}
	}
	else
	{
		NodeCopy.FilterState = InSourceNode.FilterState->DeepCopy();
	}

	return NodeCopyPtr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FFilterConfiguratorNode::operator==(const FFilterConfiguratorNode& Other) const
{
	bool bIsEqual = true;

	if (!IsGroup())
	{
		check(FilterState.IsValid());
		check(Other.FilterState.IsValid());

		if (FilterState->GetTypeName() == Other.FilterState->GetTypeName())
		{
			bIsEqual &= FilterState->Equals(*Other.FilterState);
		}
		else
		{
			return false;
		}
	}

	bIsEqual &= AvailableFilters.Get() == Other.AvailableFilters.Get();
	bIsEqual &= SelectedFilter.Get() == Other.SelectedFilter.Get();
	bIsEqual &= SelectedFilterGroupOperator.Get() == Other.SelectedFilterGroupOperator.Get();
	bIsEqual &= TextBoxValue == Other.TextBoxValue;
	bIsEqual &= GetChildrenCount() == Other.GetChildrenCount();

	if (bIsEqual)
	{
		const int32 Count = GetChildrenCount();
		for (int32 Index = 0; Index < Count; ++Index)
		{
			bIsEqual &= *StaticCastSharedPtr<FFilterConfiguratorNode>(GetChildren()[Index]) == *StaticCastSharedPtr<FFilterConfiguratorNode>(Other.GetChildren()[Index]);
		}
	}

	return bIsEqual;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TArray<TSharedPtr<FFilterGroupOperator>>& FFilterConfiguratorNode::GetFilterGroupOperators() const
{
	return FFilterService::Get()->GetFilterGroupOperators();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFilterConfiguratorNode::SetAvailableFilters(TSharedPtr<TArray<TSharedPtr<FFilter>>> InAvailableFilters)
{
	AvailableFilters = InAvailableFilters;

	if (AvailableFilters.IsValid() && AvailableFilters->Num() > 0)
	{
		SetSelectedFilter(AvailableFilters->GetData()[0]);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFilterConfiguratorNode::SetSelectedFilter(TSharedPtr<FFilter> InSelectedFilter)
{
	if (SelectedFilter.Get() == InSelectedFilter.Get())
	{
		return;
	}

	TextBoxValue.Empty();
	SelectedFilter = InSelectedFilter;
	check(SelectedFilter.IsValid());

	FilterState = SelectedFilter->BuildFilterState();

	if (SelectedFilter->GetSupportedOperators()->Num() > 0)
	{
		SetSelectedFilterOperator(SelectedFilter->GetSupportedOperators()->GetData()[0]);

		AvailableFilterOperators->Empty();
		SupportedOperatorsArrayConstPtr AvailableOperators = InSelectedFilter->GetSupportedOperators();
		for (auto& FilterOperator : *AvailableOperators)
		{
			AvailableFilterOperators->Add(FilterOperator);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFilterConfiguratorNode::ProcessFilter()
{
	if (IsGroup())
	{
		TArray<FBaseTreeNodePtr>& ChildArray = GetChildrenMutable();
		for (FBaseTreeNodePtr& Child : ChildArray)
		{
			FFilterConfiguratorNodePtr CastedChild = StaticCastSharedPtr<FFilterConfiguratorNode>(Child);
			CastedChild->ProcessFilter();
		}
	}
	else
	{
		if (FilterState.IsValid())
		{
			FilterState->SetFilterValue(TextBoxValue);
			FilterState->Update();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFilterConfiguratorNode::GetUsedKeys(TSet<int32>& KeysUsed) const
{
	if (IsGroup())
	{
		auto& ChildrenArr = GetChildren();
		for (int Index = 0; Index < ChildrenArr.Num(); ++Index)
		{
			((FFilterConfiguratorNode*)ChildrenArr[Index].Get())->GetUsedKeys(KeysUsed);
		}
	}
	else
	{
		KeysUsed.Add(SelectedFilter->GetKey());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FFilterConfiguratorNode::ApplyFilters(const FBaseFilterContext& Context) const
{
	if (IsGroup())
	{
		switch (SelectedFilterGroupOperator->GetType())
		{
		case EFilterGroupOperator::And:
		{
			bool Ret = true;
			auto& ChildrenArr = GetChildren();
			for (int Index = 0; Index < ChildrenArr.Num() && Ret; ++Index)
			{
				Ret &= ((FFilterConfiguratorNode*)ChildrenArr[Index].Get())->ApplyFilters(Context);
			}
			return Ret;
		}
		case EFilterGroupOperator::Or:
		{
			bool Ret = false;
			auto& ChildrenArr = GetChildren();
			for (int Index = 0; Index < ChildrenArr.Num() && !Ret; ++Index)
			{
				Ret |= ((FFilterConfiguratorNode*)ChildrenArr[Index].Get())->ApplyFilters(Context);
			}
			return Ret;
		}
		default:
			return false;
		}
	}
	else
	{
		return FilterState->ApplyFilter(Context);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFilterConfiguratorNode::Update()
{
	if (IsGroup())
	{
		TArray<FBaseTreeNodePtr>& ChildArray = GetChildrenMutable();
		for (FBaseTreeNodePtr& Child : ChildArray)
		{
			FFilterConfiguratorNodePtr CastedChild = StaticCastSharedPtr<FFilterConfiguratorNode>(Child);
			CastedChild->Update();
		}
	}
	else
	{
		if (FilterState.IsValid())
		{
			FilterState->Update();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFilterConfiguratorNode::SetTextBoxValue(const FString& InValue)
{
	TextBoxValue = InValue;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFilterConfiguratorNode::SetSelectedFilterOperator(TSharedPtr<IFilterOperator> InSelectedFilterOperator)
{
	if (FilterState.IsValid())
	{
		FilterState->SetSelectedOperator(InSelectedFilterOperator);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<const IFilterOperator> FFilterConfiguratorNode::GetSelectedFilterOperator() const
{
	if (FilterState.IsValid())
	{
		return FilterState->GetSelectedOperator();
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
