// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Filter/ViewModels/Filters.h"

#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Widgets/Docking/SDockTab.h"

#include "InsightsCore/Common/InsightsCoreStyle.h"
#include "InsightsCore/Filter/ViewModels/FilterConfigurator.h"
#include "InsightsCore/Filter/Widgets/SAdvancedFilter.h"

#define LOCTEXT_NAMESPACE "UE::Insights::Filters"

namespace UE::Insights
{

FString FBaseFilterContext::EmptyString;

INSIGHTS_IMPLEMENT_RTTI(FFilterState)
INSIGHTS_IMPLEMENT_RTTI(FFilter)
INSIGHTS_IMPLEMENT_RTTI(FFilterWithSuggestions)

////////////////////////////////////////////////////////////////////////////////////////////////////
// FFilterStorage
////////////////////////////////////////////////////////////////////////////////////////////////////

FFilterStorage::FFilterStorage()
{
	DoubleOperators = MakeShared<TArray<TSharedPtr<IFilterOperator>>>();
	DoubleOperators->Add(MakeShared<FFilterOperator<double>>(EFilterOperator::Eq, TEXT("="), std::equal_to<>()));
	DoubleOperators->Add(MakeShared<FFilterOperator<double>>(EFilterOperator::NotEq, TEXT("!="), std::not_equal_to<>()));
	DoubleOperators->Add(MakeShared<FFilterOperator<double>>(EFilterOperator::Lt, TEXT("<"), std::less<>{}));
	DoubleOperators->Add(MakeShared<FFilterOperator<double>>(EFilterOperator::Lte, TEXT("\u2264"), std::less_equal<>()));
	DoubleOperators->Add(MakeShared<FFilterOperator<double>>(EFilterOperator::Gt, TEXT(">"), std::greater<>()));
	DoubleOperators->Add(MakeShared<FFilterOperator<double>>(EFilterOperator::Gte, TEXT("\u2265"), std::greater_equal<>()));

	IntegerOperators = MakeShared<TArray<TSharedPtr<IFilterOperator>>>();
	IntegerOperators->Add(MakeShared<FFilterOperator<int64>>(EFilterOperator::Eq, TEXT("="), std::equal_to<>()));
	IntegerOperators->Add(MakeShared<FFilterOperator<int64>>(EFilterOperator::NotEq, TEXT("!="), std::not_equal_to<>()));
	IntegerOperators->Add(MakeShared<FFilterOperator<int64>>(EFilterOperator::Lt, TEXT("<"), std::less<>{}));
	IntegerOperators->Add(MakeShared<FFilterOperator<int64>>(EFilterOperator::Lte, TEXT("\u2264"), std::less_equal<>()));
	IntegerOperators->Add(MakeShared<FFilterOperator<int64>>(EFilterOperator::Eq, TEXT("="), std::equal_to<>()));
	IntegerOperators->Add(MakeShared<FFilterOperator<int64>>(EFilterOperator::Gt, TEXT(">"), std::greater<>()));
	IntegerOperators->Add(MakeShared<FFilterOperator<int64>>(EFilterOperator::Gte, TEXT("\u2265"), std::greater_equal<>()));

	StringOperators = MakeShared<TArray<TSharedPtr<IFilterOperator>>>();
	StringOperators->Add(MakeShared<FFilterOperator<FStringView>>(EFilterOperator::Eq, TEXT("IS"), [](FStringView lhs, FStringView rhs) { return lhs.Equals(rhs, ESearchCase::IgnoreCase); }));
	StringOperators->Add(MakeShared<FFilterOperator<FStringView>>(EFilterOperator::NotEq, TEXT("IS NOT"), [](FStringView lhs, FStringView rhs) { return !lhs.Equals(rhs, ESearchCase::IgnoreCase); }));
	StringOperators->Add(MakeShared<FFilterOperator<FStringView>>(EFilterOperator::Contains, TEXT("CONTAINS"), [](FStringView lhs, FStringView rhs) { return lhs.Contains(rhs, ESearchCase::IgnoreCase); }));
	StringOperators->Add(MakeShared<FFilterOperator<FStringView>>(EFilterOperator::NotContains, TEXT("NOT CONTAINS"), [](FStringView lhs, FStringView rhs) { return !lhs.Contains(rhs, ESearchCase::IgnoreCase); }));

	FilterGroupOperators.Add(MakeShared<FFilterGroupOperator>(EFilterGroupOperator::And, LOCTEXT("AllOf", "All Of (AND)"), LOCTEXT("AllOfDesc", "All of the children must be true for the group to return true. Equivalent to an AND operation.")));
	FilterGroupOperators.Add(MakeShared<FFilterGroupOperator>(EFilterGroupOperator::Or, LOCTEXT("AnyOf", "Any Of (OR)"), LOCTEXT("AnyOfDesc", "Any of the children must be true for the group to return true. Equivalent to an OR operation.")));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FFilterStorage::~FFilterStorage()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FFilterService
////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FFilterService> FFilterService::Instance;

FName const FFilterService::FilterConfiguratorTabId(TEXT("InsightsFilterConfigurator"));

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFilterService::Initialize()
{
	check(!Instance.IsValid());
	Instance = MakeShared<FFilterService>();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFilterService::Shutdown()
{
	Instance = nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FFilterService::FFilterService()
{
	RegisterTabSpawner();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FFilterService::~FFilterService()
{
	UnregisterTabSpawner();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SWidget> FFilterService::CreateFilterConfiguratorWidget(TSharedPtr<FFilterConfigurator> FilterConfiguratorViewModel, TWeakPtr<SWidget> ParentWidget)
{
	SAssignNew(PendingWidget, SAdvancedFilter, FilterConfiguratorViewModel);
	PendingParentWidget = ParentWidget;

	if (FGlobalTabmanager::Get()->HasTabSpawner(FilterConfiguratorTabId))
	{
		FGlobalTabmanager::Get()->TryInvokeTab(FilterConfiguratorTabId);
	}

	return PendingWidget;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SDockTab> FFilterService::SpawnTab(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab);

	TSharedPtr<SWidget> ParentWidget = PendingParentWidget.Pin();
	const TSharedPtr<SWindow>& OwnerWindow = Args.GetOwnerWindow();
	if (OwnerWindow.IsValid() && ParentWidget.IsValid() && OwnerWindow != FSlateApplication::Get().FindWidgetWindow(ParentWidget.ToSharedRef()))
	{
		TSharedPtr<SWindow> TopmostAncestor = OwnerWindow->GetTopmostAncestor();
		if (TopmostAncestor != OwnerWindow)
		{
			const FVector2D DPIProbePoint = TopmostAncestor->GetPositionInScreen() + FVector2D(10.0, 10.0);
			const float LocalDPIScaleFactor = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(static_cast<float>(DPIProbePoint.X), static_cast<float>(DPIProbePoint.Y));
			OwnerWindow->Resize(FVector2D(600 * LocalDPIScaleFactor, 400 * LocalDPIScaleFactor));
		}
	}

	check(PendingWidget.IsValid());
	DockTab->SetContent(PendingWidget.ToSharedRef());
	PendingWidget->SetParentTab(DockTab);

	PendingWidget = nullptr;
	PendingParentWidget = nullptr;
	return DockTab;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFilterService::RegisterTabSpawner()
{
	FTabSpawnerEntry& TabSpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(FilterConfiguratorTabId,
		FOnSpawnTab::CreateRaw(this, &FFilterService::SpawnTab))
		.SetDisplayName(LOCTEXT("FilterConfiguratorTabTitle", "Filter Configurator"))
		.SetMenuType(ETabSpawnerMenuType::Hidden)
		.SetIcon(FSlateIcon(FInsightsCoreStyle::GetStyleSetName(), "Icons.ClassicFilterConfig"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFilterService::UnregisterTabSpawner()
{
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(FilterConfiguratorTabId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FFilterState
////////////////////////////////////////////////////////////////////////////////////////////////////

void FFilterState::SetFilterValue(FString InTextValue)
{
	if (InTextValue.IsEmpty())
	{
		FilterValue = FFilterContext::ContextData();
	}

	switch (Filter->GetDataType())
	{
	case EFilterDataType::Double:
	{
		if (Filter->GetConverter().IsValid())
		{
			double Value = 0.0;
			FText Errors;
			bool Result = Filter->GetConverter()->Convert(InTextValue, Value, Errors);
			FilterValue.Set<double>(Result ? Value : 0.0);
		}
		else
		{
			FilterValue.Set<double>(FCString::Atod(*InTextValue));
		}
		break;
	}
	case EFilterDataType::Int64:
	{
		if (Filter->GetConverter().IsValid())
		{
			int64 Value = 0;
			FText Errors;
			bool Result = Filter->GetConverter()->Convert(InTextValue, Value, Errors);
			FilterValue.Set<int64>(Result ? Value : 0);
		}
		else
		{
			if (InTextValue.Contains(TEXT("x")))
			{
				FilterValue.Set<int64>((int64)FParse::HexNumber64(*InTextValue));
			}
			else
			{
				FilterValue.Set<int64>(FCString::Atoi64(*InTextValue));
			}
		}
		break;
	}
	case EFilterDataType::String:
	{
		FilterValue.Set<FString>(InTextValue);
		break;
	}
	case EFilterDataType::StringInt64Pair:
	{
		checkf(Filter->GetConverter().IsValid(), TEXT("StringToInt64Pair filters must have a converter set"));
		int64 Value = 0;
		FText Errors;
		bool Result = Filter->GetConverter()->Convert(InTextValue, Value, Errors);
		FilterValue.Set<int64>(Result ? Value : -1);
	}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FFilterState::ApplyFilter(const FBaseFilterContext& Context) const
{
	if (!Context.HasData(Filter->GetKey()))
	{
		// If data is not set for this filter return the value specified in the Context.
		return Context.GetDefaultReturnValue();
	}

	bool Ret = true;

	switch (Filter->GetDataType())
	{
	case EFilterDataType::Double:
	{
		FFilterOperator<double>* Operator = (FFilterOperator<double>*) SelectedOperator.Get();
		double Value = Context.GetDataAsDouble(Filter->GetKey());
		Ret = Operator->Apply(Value, FilterValue.Get<double>());
		break;
	}
	case EFilterDataType::Int64:
	case EFilterDataType::StringInt64Pair:
	{
		FFilterOperator<int64>* Operator = (FFilterOperator<int64>*) SelectedOperator.Get();
		int64 Value = Context.GetDataAsInt64(Filter->GetKey());
		Ret = Operator->Apply(Value, FilterValue.Get<int64>());
		break;
	}
	case EFilterDataType::String:
	{
		FFilterOperator<FStringView>* Operator = (FFilterOperator<FStringView>*) SelectedOperator.Get();
		const FString& Value = Context.GetDataAsString(Filter->GetKey());
		Ret = Operator->Apply(Value, FilterValue.Get<FString>());
		break;
	}
	default:
		break;
	}

	return Ret;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FFilterState::Equals(const FFilterState& Other) const
{
	return SelectedOperator == Other.GetSelectedOperator();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FFilterState> FFilterState::DeepCopy() const
{
	return MakeShared<FFilterState>(*this);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FFilterWithSuggestionsValueConverter
////////////////////////////////////////////////////////////////////////////////////////////////////

FText FFilterWithSuggestionsValueConverter::GetTooltipText() const
{
	return LOCTEXT("FilterWithSuggestionsValueConverterTooltip", "Enter the value to search for. This field has auto-complete. Start typing or press the arrow down or arrow up key to see options.");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FFilterWithSuggestionsValueConverter::GetHintText() const
{
	return LOCTEXT("FilterWithSuggestionsValueConverterHint", "Start typing or press arrow down or up to see options.");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
