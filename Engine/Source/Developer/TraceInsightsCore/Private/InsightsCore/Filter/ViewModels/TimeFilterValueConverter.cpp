// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Filter/ViewModels/TimeFilterValueConverter.h"

#include "Internationalization/Internationalization.h"
#include "Misc/CString.h"

#define LOCTEXT_NAMESPACE "UE::Insights::FTimeFilterValueConverter"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTimeFilterValueConverter::Convert(const FString& Input, double& Output, FText& OutError) const
{
	FString TrimmedInput = Input.TrimStartAndEnd();
	int Index = 0;
	for (; Index < TrimmedInput.Len(); ++Index)
	{
		TCHAR Character = TrimmedInput[Index];
		if (!TChar<TCHAR>::IsDigit(Character) && !(Character == '.'))
		{
			break;
		}
	}

	FString Value = TrimmedInput.Left(Index);
	if (!Value.IsNumeric())
	{
		OutError = LOCTEXT("InvalidValue", "Invalid value!");
	}
	Output = FCString::Atod(*Value);

	FString Unit = TrimmedInput.RightChop(Index);
	Unit.TrimStartAndEndInline();
	if (Unit.Len() == 0)
	{
		// The default unit if one is not specified by the user.
		Unit = TEXT("s");
	}

	if (Unit.Compare(TEXT("ps"), ESearchCase::IgnoreCase) == 0)
	{
		Output *= 1.e-12f;
		return true;
	}
	else if (Unit.Compare(TEXT("ns"), ESearchCase::IgnoreCase) == 0)
	{
		Output *= 1.e-9f;
		return true;
	}
	else if (Unit.Compare(TEXT("µs"), ESearchCase::IgnoreCase) == 0 || Unit.Compare(TEXT("us"), ESearchCase::IgnoreCase) == 0)
	{
		Output *= 1.e-6f;
		return true;
	}
	else if (Unit.Compare(TEXT("ms"), ESearchCase::IgnoreCase) == 0)
	{
		Output *= 1.e-3f;
		return true;
	}
	else if (Unit.Compare(TEXT("s"), ESearchCase::IgnoreCase) == 0)
	{
		return true;
	}
	else if (Unit.Compare(TEXT("m"), ESearchCase::IgnoreCase) == 0)
	{
		Output *= 60;
		return true;
	}
	else if (Unit.Compare(TEXT("h"), ESearchCase::IgnoreCase) == 0)
	{
		Output *= 60 * 60;
		return true;
	}
	else if (Unit.Compare(TEXT("d"), ESearchCase::IgnoreCase) == 0)
	{
		Output *= 60 * 60 * 24;
		return true;
	}

	OutError = LOCTEXT("InvalidUnit", "Invalid unit! Can be one of: ps,ns,µs/us,ms,s,m,d.");
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FTimeFilterValueConverter::GetTooltipText() const
{
	return LOCTEXT("FTimeFilterValueConverterTooltip", "Enter a time value as a floating point number followed by an optional unit (ps,ns,µs/us,ms,s,m,d).\nThe default unit is \"s\" (seconds).");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FTimeFilterValueConverter::GetHintText() const
{
	// An example value.
	return FText::FromString(TEXT("1.5s"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
