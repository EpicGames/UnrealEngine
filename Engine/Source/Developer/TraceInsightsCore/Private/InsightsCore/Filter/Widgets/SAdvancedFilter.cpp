// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Filter/Widgets/SAdvancedFilter.h"

#include "SlateOptMacros.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableViewBase.h"

#include "InsightsCore/Filter/Widgets/SFilterConfigurator.h"
#include "InsightsCore/Filter/ViewModels/FilterConfigurator.h"

#define LOCTEXT_NAMESPACE "UE::Insights::SAdvancedFilter"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// SAdvancedFilter
////////////////////////////////////////////////////////////////////////////////////////////////////

SAdvancedFilter::SAdvancedFilter()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SAdvancedFilter::~SAdvancedFilter()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SAdvancedFilter::InitCommandList()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SAdvancedFilter::Construct(const FArguments& InArgs, TSharedPtr<FFilterConfigurator> InFilterConfiguratorViewModel)
{
	FilterConfiguratorViewModel = MakeShared<FFilterConfigurator>(*InFilterConfiguratorViewModel);

	ChildSlot
	[
		SNew(SVerticalBox)

		// Tree view
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(0.0f, 0.0f, 0.0f, 0.0f)
		[
			SAssignNew(FilterConfigurator, SFilterConfigurator, FilterConfiguratorViewModel)
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.0f, 4.0f, 4.0f, 4.0f)
		.HAlign(EHorizontalAlignment::HAlign_Right)
		.VAlign(EVerticalAlignment::VAlign_Bottom)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(0.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SButton)
				.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("PrimaryButton"))
				.Text(LOCTEXT("OK", "OK"))
				.ToolTipText(LOCTEXT("OKDesc", "Apply the changes to the filters."))
				.OnClicked(this, &SAdvancedFilter::OK_OnClicked)
			]

			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SButton)
				.Text(LOCTEXT("Cancel", "Cancel"))
				.ToolTipText(LOCTEXT("CancelDesc", "Cancel the changes to the filter."))
				.OnClicked(this, &SAdvancedFilter::Cancel_OnClicked)
			]
		]
	];

	SHeaderRow::FColumn::FArguments ColumnArgs;
	ColumnArgs
		.ColumnId(TEXT("Filter"))
		.DefaultLabel(LOCTEXT("FilterColumnHeader", "Filter"))
		.HAlignHeader(HAlign_Fill)
		.VAlignHeader(VAlign_Fill)
		.HeaderContentPadding(FMargin(2.0f))
		.HAlignCell(HAlign_Fill)
		.VAlignCell(VAlign_Fill)
		.HeaderContent()
		[
			SNew(SBox)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("FilterColumnHeader", "Filter"))
			]
		];

	OnViewModelDestroyedHandle = InFilterConfiguratorViewModel->GetOnDestroyedEvent().AddSP(this, &SAdvancedFilter::RequestClose);
	OriginalFilterConfiguratorViewModel = InFilterConfiguratorViewModel;
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SAdvancedFilter::OK_OnClicked()
{
	TSharedPtr<FFilterConfigurator> OriginalFilterVM = OriginalFilterConfiguratorViewModel.Pin();
	if (OriginalFilterVM.IsValid())
	{
		*OriginalFilterVM = *FilterConfiguratorViewModel;
	}

	RequestClose();

	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SAdvancedFilter::Cancel_OnClicked()
{
	RequestClose();

	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SAdvancedFilter::RequestClose()
{
	TSharedPtr<SDockTab> ParentTabSharedPtr = ParentTab.Pin();
	if (ParentTabSharedPtr.IsValid())
	{
		ParentTabSharedPtr->RequestCloseTab();
		ParentTabSharedPtr.Reset();
	}

	TSharedPtr<FFilterConfigurator> OriginalFilterVM = OriginalFilterConfiguratorViewModel.Pin();
	if (OriginalFilterVM.IsValid())
	{
		OriginalFilterVM->GetOnDestroyedEvent().Remove(OnViewModelDestroyedHandle);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
