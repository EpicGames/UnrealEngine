// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Filter/Widgets/SFilterConfigurator.h"

#include "SlateOptMacros.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableViewBase.h"

#include "InsightsCore/Filter/Widgets/SFilterConfiguratorRow.h"
#include "InsightsCore/Filter/ViewModels/FilterConfigurator.h"

#define LOCTEXT_NAMESPACE "UE::Insights::SFilterConfigurator"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// SFilterConfigurator
////////////////////////////////////////////////////////////////////////////////////////////////////

SFilterConfigurator::SFilterConfigurator()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SFilterConfigurator::~SFilterConfigurator()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFilterConfigurator::InitCommandList()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SFilterConfigurator::Construct(const FArguments& InArgs, TSharedPtr<FFilterConfigurator> InFilterConfiguratorViewModel)
{
	ChildSlot
	[
		SNew(SVerticalBox)

		// Tree view
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(0.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SBox)
			.HAlign(HAlign_Fill)
			.Padding(0.0f)
			[
				SAssignNew(TreeView, STreeView<FFilterConfiguratorNodePtr>)
				.SelectionMode(ESelectionMode::None)
				.TreeItemsSource(&GroupNodes)
				.OnGetChildren(this, &SFilterConfigurator::TreeView_OnGetChildren)
				.OnGenerateRow(this, &SFilterConfigurator::TreeView_OnGenerateRow)
				.HeaderRow
				(
					SAssignNew(TreeViewHeaderRow, SHeaderRow)
					.Visibility(EVisibility::Visible)
				)
			]
		]
	];

	SHeaderRow::FColumn::FArguments ColumnArgs;
	ColumnArgs
		.ColumnId(TEXT("Filter"))
		.DefaultLabel(LOCTEXT("FilterColumnHeader", "Filter"))
		.HAlignHeader(HAlign_Fill)
		.VAlignHeader(VAlign_Fill)
		.HeaderContentPadding(FMargin(2.0f))
		.HAlignCell(HAlign_Fill)
		.VAlignCell(VAlign_Fill)
		.HeaderContent()
		[
			SNew(SBox)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("FilterColumnHeader", "Filter"))
			]
		];

	TreeViewHeaderRow->InsertColumn(ColumnArgs, 0);

	FilterConfiguratorViewModel = InFilterConfiguratorViewModel;

	GroupNodes.Add(FilterConfiguratorViewModel->GetRootNode());

	SetInitialExpansionRec(FilterConfiguratorViewModel->GetRootNode(), true);
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFilterConfigurator::TreeView_OnGetChildren(FFilterConfiguratorNodePtr InParent, TArray<FFilterConfiguratorNodePtr>& OutChildren)
{
	const TArray<FBaseTreeNodePtr>& Children = InParent->GetChildren();
	OutChildren.Reset(Children.Num());
	for (const FBaseTreeNodePtr& Child : Children)
	{
		OutChildren.Add(StaticCastSharedPtr<FFilterConfiguratorNode, FBaseTreeNode>(Child));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<ITableRow> SFilterConfigurator::TreeView_OnGenerateRow(FFilterConfiguratorNodePtr TreeNode, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<ITableRow> TableRow =
		SNew(SFilterConfiguratorRow, OwnerTable)
		.FilterConfiguratorNodePtr(TreeNode);

	return TableRow;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFilterConfigurator::SetInitialExpansionRec(const FFilterConfiguratorNodePtr& Node, bool Value)
{
	TreeView->SetItemExpansion(Node, Value);
	for (const FBaseTreeNodePtr& Child : Node->GetChildren())
	{
		SetInitialExpansionRec(StaticCastSharedPtr<FFilterConfiguratorNode, FBaseTreeNode>(Child), Value);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
