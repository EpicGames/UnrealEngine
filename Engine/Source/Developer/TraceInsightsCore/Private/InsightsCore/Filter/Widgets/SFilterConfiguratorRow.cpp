// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Filter/Widgets/SFilterConfiguratorRow.h"

#include "Framework/Application/SlateApplication.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSuggestionTextBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

#include "InsightsCore/Common/InsightsCoreStyle.h"
#include "InsightsCore/Filter/ViewModels/Filters.h"

#define LOCTEXT_NAMESPACE "UE::Insights::SFilterConfiguratorRow"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SFilterConfiguratorRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	FilterConfiguratorNodePtr = InArgs._FilterConfiguratorNodePtr;
	SMultiColumnTableRow<FFilterConfiguratorNodePtr>::Construct(SMultiColumnTableRow<FFilterConfiguratorNodePtr>::FArguments(), InOwnerTableView);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SFilterConfiguratorRow::GenerateWidgetForColumn(const FName& InColumnName)
{
	TSharedRef<SHorizontalBox> LeftBox = SNew(SHorizontalBox);
	TSharedRef<SHorizontalBox> RightBox = SNew(SHorizontalBox);

	if (!FilterConfiguratorNodePtr->IsGroup())
	{
		LeftBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Fill)
			.Padding(FMargin(0.0f))
			[
				SNew(SExpanderArrow, SharedThis(this))
				.ShouldDrawWires(true)
			];

		TSharedPtr<FFilterState> FilterState = FilterConfiguratorNodePtr->GetSelectedFilterState();
		check(FilterState.IsValid());

		// Filter combo box
		LeftBox->AddSlot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(FMargin(0.0f, 2.0f))
			[
				SAssignNew(FilterTypeComboBox, SComboBox<TSharedPtr<FFilter>>)
				.OptionsSource(GetAvailableFilters())
				.OnSelectionChanged(this, &SFilterConfiguratorRow::AvailableFilters_OnSelectionChanged)
				.OnGenerateWidget(this, &SFilterConfiguratorRow::AvailableFilters_OnGenerateWidget)
				[
					SNew(STextBlock)
					.Text(this, &SFilterConfiguratorRow::AvailableFilters_GetSelectionText)
				]
			];

		if (!FilterState->HasCustomUI())
		{
			// Operator combo box
			LeftBox->AddSlot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.FillWidth(1.0)
				.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
				.AutoWidth()
				[
					SAssignNew(FilterOperatorComboBox, SComboBox<TSharedPtr<IFilterOperator>>)
					.OptionsSource(GetAvailableFilterOperators())
					.OnSelectionChanged(this, &SFilterConfiguratorRow::AvailableFilterOperators_OnSelectionChanged)
					.OnGenerateWidget(this, &SFilterConfiguratorRow::AvailableFilterOperators_OnGenerateWidget)
					[
						SNew(STextBlock)
						.Text(this, &SFilterConfiguratorRow::AvailableFilterOperators_GetSelectionText)
					]
				];

			if (FilterConfiguratorNodePtr->GetSelectedFilter()->Is<FFilterWithSuggestions>())
			{
				SuggestionTextBoxValue = FilterConfiguratorNodePtr->GetTextBoxValue();

				LeftBox->AddSlot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					.Padding(FMargin(4.0f, 0.0f, 4.0f, 0.0f))
					[
						SNew(SSuggestionTextBox)
						.MinDesiredWidth(300.0f)
						.ForegroundColor(FSlateColor::UseForeground())
						.OnTextCommitted(this, &SFilterConfiguratorRow::SuggestionTextBox_OnValueCommitted)
						.OnTextChanged(this, &SFilterConfiguratorRow::SuggestionTextBox_OnValueChanged)
						.Text(this, &SFilterConfiguratorRow::SuggestionTextBox_GetValue)
						.ToolTipText(this, &SFilterConfiguratorRow::GetTextBoxTooltipText)
						.HintText(this, &SFilterConfiguratorRow::GetTextBoxHintText)
						.OnShowingSuggestions(FOnShowingSuggestions::CreateSP(this, &SFilterConfiguratorRow::SuggestionTextBox_GetSuggestions))
						.OnShowingHistory(FOnShowingHistory::CreateSP(this, &SFilterConfiguratorRow::SuggestionTextBox_GetHistory))
					];
			}
			else
			{
				LeftBox->AddSlot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					.FillWidth(1.0)
					.Padding(FMargin(4.0f, 0.0f, 4.0f, 0.0f))
					[
						SNew(SEditableTextBox)
						.MinDesiredWidth(50.0f)
						.OnTextCommitted(this, &SFilterConfiguratorRow::OnTextBoxValueCommitted)
						.Text(this, &SFilterConfiguratorRow::GetTextBoxValue)
						.ToolTipText(this, &SFilterConfiguratorRow::GetTextBoxTooltipText)
						.HintText(this, &SFilterConfiguratorRow::GetTextBoxHintText)
						.OnVerifyTextChanged(this, &SFilterConfiguratorRow::TextBox_OnVerifyTextChanged)
					];
			}
		}
		else
		{
			FilterState->AddCustomUI(LeftBox);
		}

		RightBox->AddSlot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.AutoWidth()
			.Padding(FMargin(2.0f, 0.0f))
			[
				SNew(SButton)
				.ToolTipText(LOCTEXT("DeleteFilterTooptip", "Delete Filter"))
				.OnClicked(this, &SFilterConfiguratorRow::DeleteFilter_OnClicked)
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
				]
			];
	}
	else
	{
		LeftBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Fill)
			.Padding(FMargin(0.0f))
			[
				SNew(SExpanderArrow, SharedThis(this))
				.ShouldDrawWires(true)
			];

		LeftBox->AddSlot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(FMargin(0.0f, 2.0f))
			[
				SAssignNew(FilterGroupOperatorComboBox, SComboBox<TSharedPtr<FFilterGroupOperator>>)
				.OptionsSource(GetFilterGroupOperators())
				.OnSelectionChanged(this, &SFilterConfiguratorRow::FilterGroupOperators_OnSelectionChanged)
				.OnGenerateWidget(this, &SFilterConfiguratorRow::FilterGroupOperators_OnGenerateWidget)
				[
					SNew(STextBlock)
					.Text(this, &SFilterConfiguratorRow::FilterGroupOperators_GetSelectionText)
					.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
				]
			];

		RightBox->AddSlot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(FMargin(2.0f, 0.0f))
			[
				SNew(SButton)
				.ToolTipText(LOCTEXT("AddFilterDesc", "Add a filter node as a child to this group node."))
				.OnClicked(this, &SFilterConfiguratorRow::AddFilter_OnClicked)
				.Content()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Filter"))
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.Margin(FMargin(2.0, 0.0f, 0.0f, 0.0f))
						.Text(LOCTEXT("AddFilter", "Add Filter"))
					]
				]
			];

		RightBox->AddSlot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(FMargin(2.0f, 0.0f))
			[
				SNew(SButton)
				.ToolTipText(LOCTEXT("AddGroupDesc", "Add a group node as a child to this group node."))
				.OnClicked(this, &SFilterConfiguratorRow::AddGroup_OnClicked)
				.Content()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FInsightsCoreStyle::GetBrush("Icons.FilterAddGroup"))
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.Margin(FMargin(2.0, 0.0f, 0.0f, 0.0f))
						.Text(LOCTEXT("AddGroup", "Add Group"))
					]
				]
			];

		// Do not show Delete button for the Root node
		EVisibility DeleteVisibility = FilterConfiguratorNodePtr->GetParent().IsValid() ? EVisibility::Visible : EVisibility::Hidden;
		RightBox->AddSlot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.AutoWidth()
			.Padding(FMargin(2.0f, 0.0f))
			[
				SNew(SButton)
				.ToolTipText(LOCTEXT("DeleteGroupTooptip", "Delete Group"))
				.OnClicked(this, &SFilterConfiguratorRow::DeleteGroup_OnClicked)
				.HAlign(HAlign_Right)
				.Visibility(DeleteVisibility)
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
				]
			];
	}

	TSharedRef<SHorizontalBox> GeneratedWidget = SNew(SHorizontalBox);

	GeneratedWidget->AddSlot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		[
			LeftBox
		];

	GeneratedWidget->AddSlot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		.FillWidth(1.0f)
		[
			RightBox
		];

	return GeneratedWidget;
}

///////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SFilterConfiguratorRow::AvailableFilters_OnGenerateWidget(TSharedPtr<FFilter> InFilter)
{
	TSharedRef<SHorizontalBox> Widget = SNew(SHorizontalBox);
	Widget->AddSlot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(InFilter->GetName())
			.Margin(2.0f)
		];

	return Widget;
}

///////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SFilterConfiguratorRow::AvailableFilterOperators_OnGenerateWidget(TSharedPtr<IFilterOperator> InFilterOperator)
{
	TSharedRef<SHorizontalBox> Widget = SNew(SHorizontalBox);
	Widget->AddSlot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(FText::FromString(InFilterOperator->GetName()))
			.Margin(2.0f)
		];

	return Widget;
}

///////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SFilterConfiguratorRow::FilterGroupOperators_OnGenerateWidget(TSharedPtr<FFilterGroupOperator> InFilterGroupOperator)
{
	TSharedRef<SHorizontalBox> Widget = SNew(SHorizontalBox);
	Widget->SetToolTipText(InFilterGroupOperator->GetDesc());
	Widget->AddSlot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(InFilterGroupOperator->GetName())
			.Margin(2.0f)
		];

	return Widget;
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

const TArray<TSharedPtr<FFilter>>* SFilterConfiguratorRow::GetAvailableFilters()
{
	return FilterConfiguratorNodePtr->GetAvailableFilters().Get();
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void SFilterConfiguratorRow::AvailableFilters_OnSelectionChanged(TSharedPtr<FFilter> InFilter, ESelectInfo::Type SelectInfo)
{
	FilterConfiguratorNodePtr->SetSelectedFilter(InFilter);

	TSharedPtr<STreeView<FFilterConfiguratorNodePtr>> OwnerTable = StaticCastSharedPtr<STreeView<FFilterConfiguratorNodePtr>>(OwnerTablePtr.Pin());
	ensure(OwnerTable.IsValid());
	ClearCellCache();
	GenerateColumns(OwnerTable->GetHeaderRow().ToSharedRef());
}

///////////////////////////////////////////////////////////////////////////////////////////////////

FText SFilterConfiguratorRow::AvailableFilters_GetSelectionText() const
{
	return FilterConfiguratorNodePtr->GetSelectedFilter()->GetName();
}

///////////////////////////////////////////////////////////////////////////////////////////////////

const TArray<TSharedPtr<IFilterOperator>>* SFilterConfiguratorRow::GetAvailableFilterOperators()
{
	return FilterConfiguratorNodePtr->GetAvailableFilterOperators().Get();
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void SFilterConfiguratorRow::AvailableFilterOperators_OnSelectionChanged(TSharedPtr<IFilterOperator> InFilter, ESelectInfo::Type SelectInfo)
{
	FilterConfiguratorNodePtr->SetSelectedFilterOperator(InFilter);
}

///////////////////////////////////////////////////////////////////////////////////////////////////

FText SFilterConfiguratorRow::AvailableFilterOperators_GetSelectionText() const
{
	return FText::FromString(FilterConfiguratorNodePtr->GetSelectedFilterOperator()->GetName());
}

///////////////////////////////////////////////////////////////////////////////////////////////////

const TArray<TSharedPtr<FFilterGroupOperator>>* SFilterConfiguratorRow::GetFilterGroupOperators()
{
	return &FilterConfiguratorNodePtr->GetFilterGroupOperators();
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void SFilterConfiguratorRow::FilterGroupOperators_OnSelectionChanged(TSharedPtr<FFilterGroupOperator> InFilterGroupOperator, ESelectInfo::Type SelectInfo)
{
	FilterConfiguratorNodePtr->SetSelectedFilterGroupOperator(InFilterGroupOperator);
}

///////////////////////////////////////////////////////////////////////////////////////////////////

FText SFilterConfiguratorRow::FilterGroupOperators_GetSelectionText() const
{
	return FilterConfiguratorNodePtr->GetSelectedFilterGroupOperator()->GetName();
}

///////////////////////////////////////////////////////////////////////////////////////////////////

FReply SFilterConfiguratorRow::AddFilter_OnClicked()
{
	FFilterConfiguratorNodePtr ChildNode = MakeShared<FFilterConfiguratorNode>(false);
	ChildNode->SetAvailableFilters(FilterConfiguratorNodePtr->GetAvailableFilters());

	FilterConfiguratorNodePtr->AddChildAndSetParent(ChildNode);
	FilterConfiguratorNodePtr->SetExpansion(true);
	OwnerTablePtr.Pin()->Private_SetItemExpansion(FilterConfiguratorNodePtr, true);

	StaticCastSharedPtr<STreeView<FFilterConfiguratorNodePtr>>(OwnerTablePtr.Pin())->RequestTreeRefresh();
	return FReply::Handled();
}

///////////////////////////////////////////////////////////////////////////////////////////////////

FReply SFilterConfiguratorRow::DeleteFilter_OnClicked()
{
	FilterConfiguratorNodePtr->RemoveFromParent();

	StaticCastSharedPtr<STreeView<FFilterConfiguratorNodePtr>>(OwnerTablePtr.Pin())->RequestTreeRefresh();
	return FReply::Handled();
}

///////////////////////////////////////////////////////////////////////////////////////////////////

FReply SFilterConfiguratorRow::AddGroup_OnClicked()
{
	FFilterConfiguratorNodePtr ChildNode = MakeShared<FFilterConfiguratorNode>(true);
	ChildNode->SetAvailableFilters(FilterConfiguratorNodePtr->GetAvailableFilters());
	ChildNode->SetExpansion(true);

	FilterConfiguratorNodePtr->AddChildAndSetParent(ChildNode);
	FilterConfiguratorNodePtr->SetExpansion(true);
	OwnerTablePtr.Pin()->Private_SetItemExpansion(FilterConfiguratorNodePtr, true);

	StaticCastSharedPtr<STreeView<FFilterConfiguratorNodePtr>>(OwnerTablePtr.Pin())->RequestTreeRefresh();
	return FReply::Handled();
}

///////////////////////////////////////////////////////////////////////////////////////////////////

FReply SFilterConfiguratorRow::DeleteGroup_OnClicked()
{
	FilterConfiguratorNodePtr->RemoveFromParent();

	StaticCastSharedPtr<STreeView<FFilterConfiguratorNodePtr>>(OwnerTablePtr.Pin())->RequestTreeRefresh();
	return FReply::Handled();
}

///////////////////////////////////////////////////////////////////////////////////////////////////

FText SFilterConfiguratorRow::GetTextBoxValue() const
{
	return FText::FromString(FilterConfiguratorNodePtr->GetTextBoxValue());
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void SFilterConfiguratorRow::OnTextBoxValueCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
{
	FilterConfiguratorNodePtr->SetTextBoxValue(InNewText.ToString());
}

///////////////////////////////////////////////////////////////////////////////////////////////////

FText SFilterConfiguratorRow::GetTextBoxTooltipText() const
{
	const TSharedPtr<IFilterValueConverter>& Converter = FilterConfiguratorNodePtr->GetSelectedFilter()->GetConverter();
	if (Converter.IsValid())
	{
		return Converter->GetTooltipText();
	}

	return FText();
}

///////////////////////////////////////////////////////////////////////////////////////////////////

FText SFilterConfiguratorRow::GetTextBoxHintText() const
{
	const TSharedPtr<IFilterValueConverter>& Converter = FilterConfiguratorNodePtr->GetSelectedFilter()->GetConverter();
	if (Converter.IsValid())
	{
		return Converter->GetHintText();
	}

	return FText();
}

///////////////////////////////////////////////////////////////////////////////////////////////////

bool SFilterConfiguratorRow::TextBox_OnVerifyTextChanged(const FText& InText, FText& OutErrorMessage)
{
	const TSharedPtr<IFilterValueConverter>& Converter = FilterConfiguratorNodePtr->GetSelectedFilter()->GetConverter();
	if (Converter.IsValid())
	{
		if (FilterConfiguratorNodePtr->GetSelectedFilter()->GetDataType() == EFilterDataType::Int64)
		{
			int64 Value;
			return Converter->Convert(InText.ToString(), Value, OutErrorMessage);
		}
		else if (FilterConfiguratorNodePtr->GetSelectedFilter()->GetDataType() == EFilterDataType::Double)
		{
			double Value;
			return Converter->Convert(InText.ToString(), Value, OutErrorMessage);
		}
	}

	return true;
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void SFilterConfiguratorRow::SuggestionTextBox_GetSuggestions(const FString& Text, TArray<FString>& Suggestions)
{
	TSharedPtr<FFilter> Filter = FilterConfiguratorNodePtr->GetSelectedFilter();

	if (Filter->Is<FFilterWithSuggestions>())
	{
		TSharedPtr<FFilterWithSuggestions> FilterWithSuggestions = StaticCastSharedPtr<FFilterWithSuggestions>(Filter);
		FilterWithSuggestions->GetSuggestions(Text, Suggestions);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void SFilterConfiguratorRow::SuggestionTextBox_GetHistory(TArray<FString>& Suggestions)
{
	// We show all suggestion instead of history when arrow up/down is used.
	SuggestionTextBox_GetSuggestions(FString(), Suggestions);
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void SFilterConfiguratorRow::SuggestionTextBox_OnValueChanged(const FText& InNewText)
{
	SuggestionTextBoxValue = InNewText.ToString();
}

///////////////////////////////////////////////////////////////////////////////////////////////////

FText SFilterConfiguratorRow::SuggestionTextBox_GetValue() const
{
	return FText::FromString(SuggestionTextBoxValue);
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void SFilterConfiguratorRow::SuggestionTextBox_OnValueCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
{
	FilterConfiguratorNodePtr->SetTextBoxValue(InNewText.ToString());
}

///////////////////////////////////////////////////////////////////////////////////////////////////


} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
