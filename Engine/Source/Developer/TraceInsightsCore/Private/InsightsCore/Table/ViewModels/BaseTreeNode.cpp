// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Table/ViewModels/BaseTreeNode.h"

#include "Styling/SlateBrush.h"
#include "Styling/StyleColors.h"

#include "InsightsCore/Common/InsightsCoreStyle.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"

#define LOCTEXT_NAMESPACE "UE::Insights::FBaseTreeNode"

namespace UE::Insights
{

INSIGHTS_IMPLEMENT_RTTI(FBaseTreeNode)

FBaseTreeNode::FGroupNodeData FBaseTreeNode::DefaultGroupData;

////////////////////////////////////////////////////////////////////////////////////////////////////

const FText FBaseTreeNode::GetDisplayName() const
{
	return FText::FromName(Name);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FText FBaseTreeNode::GetExtraDisplayName() const
{
	if (IsGroup())
	{
		const int32 NumChildren = GetChildrenCount();
		const int32 NumFilteredChildren = GetFilteredChildrenCount();

		if (NumFilteredChildren == NumChildren)
		{
			return FText::Format(LOCTEXT("TreeNodeGroup_ExtraText_Fmt1", "({0})"), FText::AsNumber(NumChildren));
		}
		else
		{
			return FText::Format(LOCTEXT("TreeNodeGroup_ExtraText_Fmt2", "({0} / {1})"), FText::AsNumber(NumFilteredChildren), FText::AsNumber(NumChildren));
		}
	}

	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FBaseTreeNode::HasExtraDisplayName() const
{
	return IsGroup();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* FBaseTreeNode::GetDefaultIcon(bool bIsGroupNode)
{
	if (bIsGroupNode)
	{
		return FInsightsCoreStyle::GetBrush("Icons.Group.TreeItem");
	}
	else
	{
		return FInsightsCoreStyle::GetBrush("Icons.Leaf.TreeItem");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor FBaseTreeNode::GetDefaultIconColor(bool bIsGroupNode)
{
	if (bIsGroupNode)
	{
		// USlateThemeManager::Get().GetColor(EStyleColor::AccentFolder), but lighter
		return FLinearColor(1.0f, 0.7f, 0.3f, 1.0f);
	}
	else
	{
		// USlateThemeManager::Get().GetColor(EStyleColor::AccentWhite)
		return FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor FBaseTreeNode::GetDefaultColor(bool bIsGroupNode)
{
	if (bIsGroupNode)
	{
		// USlateThemeManager::Get().GetColor(EStyleColor::AccentFolder), but lighter
		return FLinearColor(1.0f, 0.7f, 0.3f, 1.0f);
	}
	else
	{
		// USlateThemeManager::Get().GetColor(EStyleColor::AccentWhite)
		return FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FBaseTreeNode::SortChildren(const ITableCellValueSorter& Sorter, ESortMode SortMode)
{
	Sorter.Sort(GroupData->Children, SortMode);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FBaseTreeNode::SortFilteredChildren(const ITableCellValueSorter& Sorter, ESortMode SortMode)
{
	Sorter.Sort(*GroupData->FilteredChildrenPtr, SortMode);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
