// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Table/ViewModels/Table.h"

#include "Logging/MessageLog.h"

#include "InsightsCore/Table/ViewModels/TableCellValueFormatter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueGetter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"

#define LOCTEXT_NAMESPACE "UE::Insights::FTable"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTable
////////////////////////////////////////////////////////////////////////////////////////////////////

FTable::FTable()
	: DisplayName()
	, Description()
	, Columns()
	, ColumnIdToPtrMapping()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTable::~FTable()
{
	Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTable::Reset()
{
	Columns.Reset();
	ColumnIdToPtrMapping.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FTable::GetColumnPositionIndex(const FName& ColumnId) const
{
	return Columns.IndexOfByPredicate([&ColumnId](const TSharedRef<FTableColumn>& ColumnRef) -> bool { return ColumnRef->GetId() == ColumnId; });
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTable::SetColumns(const TArray<TSharedRef<FTableColumn>>& InColumns)
{
	Columns.Reset(InColumns.Num());
	ColumnIdToPtrMapping.Reset();
	for (TSharedRef<FTableColumn> ColumnRef : InColumns)
	{
		AddColumn(ColumnRef);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTable::AddColumn(TSharedRef<FTableColumn> ColumnRef)
{
	ColumnRef->SetParentTable(SharedThis(this));
	Columns.Add(ColumnRef);
	ColumnIdToPtrMapping.Add(ColumnRef->GetId(), ColumnRef);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FName FTable::GetHierarchyColumnId()
{
	static const FName HierarchyColumnId(TEXT("_Hierarchy"));
	return HierarchyColumnId;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTable::AddHierarchyColumn(int32 ColumnIndex, const TCHAR* ColumnName)
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(GetHierarchyColumnId());
	FTableColumn& Column = *ColumnRef;

	Column.SetIndex(ColumnIndex);

	const FString ColumnNameStr = ColumnName ? FString::Printf(TEXT("Hierarchy (%s)"), ColumnName) : TEXT("Hierarchy");
	const FText ColumnNameText = FText::FromString(ColumnNameStr);

	Column.SetShortName(ColumnNameText);
	Column.SetTitleName(ColumnNameText);
	//TODO: Column.SetDescription(...);

	Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeFiltered | ETableColumnFlags::IsHierarchy);

	Column.SetHorizontalAlignment(HAlign_Left);
	Column.SetInitialWidth(90.0f);

	Column.SetDataType(ETableCellDataType::CString);

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FDisplayNameValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FTextValueFormatter>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByName>(ColumnRef);
	Column.SetValueSorter(Sorter);

	AddColumn(ColumnRef);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTable::GetVisibleColumns(TArray<TSharedRef<FTableColumn>>& InArray) const
{
	for (const TSharedRef<FTableColumn>& Column : Columns)
	{
		if (Column->IsVisible())
		{
			InArray.Add(Column);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTable::GetVisibleColumnsData(const TArray<FBaseTreeNodePtr>& InNodes, const FName& LogListingName, TCHAR Separator, bool bIncludeHeaders, FString& OutData) const
{
	constexpr TCHAR LineEnd = TEXT('\n');
	constexpr TCHAR QuotationMarkBegin = TEXT('\"');
	constexpr TCHAR QuotationMarkEnd = TEXT('\"');

	TArray<TSharedRef<FTableColumn>> VisibleColumns;
	GetVisibleColumns(VisibleColumns);

	// Table headers
	if (bIncludeHeaders)
	{
		bool bIsFirstColumn = true;
		for (const TSharedRef<FTableColumn>& ColumnRef : VisibleColumns)
		{
			if (bIsFirstColumn)
			{
				bIsFirstColumn = false;
			}
			else
			{
				OutData += Separator;
			}
			FString Value = ColumnRef->GetShortName().ToString().ReplaceCharWithEscapedChar();
			int32 CharIndex;
			if (Value.FindChar(Separator, CharIndex))
			{
				OutData += QuotationMarkBegin;
				OutData += Value;
				OutData += QuotationMarkEnd;
			}
			else
			{
				OutData += Value;
			}
		}
		OutData += LineEnd;
	}

	constexpr int32 MaxRows = 100000;
	int32 NumItems = InNodes.Num();
	if (NumItems > MaxRows)
	{
		NumItems = MaxRows;

		FMessageLog ReportMessageLog((LogListingName != NAME_None) ? LogListingName : TEXT("Other"));
		ReportMessageLog.Warning(FText::Format(LOCTEXT("TooManyRows", "Too many rows selected. Only the first {0} will be copied."), NumItems));
		ReportMessageLog.Notify();
	}

	// Selected items
	for (int Index = 0; Index < NumItems; Index++)
	{
		const FBaseTreeNodePtr& Node = InNodes[Index];

		bool bIsFirstColumn = true;
		for (const TSharedRef<FTableColumn>& ColumnRef : VisibleColumns)
		{
			if (bIsFirstColumn)
			{
				bIsFirstColumn = false;
			}
			else
			{
				OutData += Separator;
			}
			FString Value = ColumnRef->GetValueAsSerializableString(*Node).ReplaceCharWithEscapedChar();
			int32 CharIndex;
			if (Value.FindChar(Separator, CharIndex))
			{
				OutData += QuotationMarkBegin;
				OutData += Value;
				OutData += QuotationMarkEnd;
			}
			else
			{
				OutData += Value;
			}
		}
		OutData += LineEnd;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
