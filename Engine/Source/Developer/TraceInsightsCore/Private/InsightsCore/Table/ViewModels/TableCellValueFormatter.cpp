// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Table/ViewModels/TableCellValueFormatter.h"

#include "Framework/Application/SlateApplication.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

#include "InsightsCore/Common/TimeUtils.h"
#include "InsightsCore/Table/ViewModels/TableCellValueGetter.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"

#include <cmath> // for std::isnan()

#define LOCTEXT_NAMESPACE "UE::Insights::FTableCellValueFormatter"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FTableCellValueFormatter::FormatValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
{
	return FormatValue(Column.GetValue(Node));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FTableCellValueFormatter::FormatValueForTooltip(const FTableColumn& Column, const FBaseTreeNode& Node) const
{
	return FormatValueForTooltip(Column.GetValue(Node));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FTableCellValueFormatter::FormatValueForGrouping(const FTableColumn& Column, const FBaseTreeNode& Node) const
{
	return FormatValueForTooltip(Column.GetValue(Node));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FTableCellValueFormatter::CopyValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
{
	return FormatValue(Column, Node);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FTableCellValueFormatter::CopyTooltip(const FTableColumn& Column, const FBaseTreeNode& Node) const
{
	return FormatValueForTooltip(Column, Node);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<IToolTip> FTableCellValueFormatter::GetCustomTooltip(const FTableColumn& Column, const FBaseTreeNode& Node) const
{
	if (Column.IsDynamic())
	{
		return SNew(SToolTip)
			.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateStatic(&FTableCellValueFormatter::GetTooltipVisibility)))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(STextBlock)
					.Text_Lambda([this, &Column, &Node]
					{ return FormatValueForTooltip(Column.GetValueGetter()->GetValue(Column, Node)); })
				]
			];
	}
	else
	{
		return SNew(SToolTip)
			.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateStatic(&FTableCellValueFormatter::GetTooltipVisibility)))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(STextBlock)
					.Text(FormatValueForTooltip(Column.GetValueGetter()->GetValue(Column, Node)))
				]
			];
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EVisibility FTableCellValueFormatter::GetTooltipVisibility()
{
	return FSlateApplication::Get().AnyMenusVisible() ? EVisibility::Collapsed : EVisibility::Visible;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FBoolValueFormatterAsTrueFalse::FormatValue(const TOptional<FTableCellValue>& InValue) const
{
	if (InValue.IsSet())
	{
		return FText::FromString(InValue.GetValue().Bool ? TEXT("True") : TEXT("False"));
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FBoolValueFormatterAsOnOff::FormatValue(const TOptional<FTableCellValue>& InValue) const
{
	if (InValue.IsSet())
	{
		return FText::FromString(InValue.GetValue().Bool ? TEXT("On") : TEXT("Off"));
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FInt64ValueFormatterAsUInt32InfiniteNumber::FormatValue(const TOptional<FTableCellValue>& InValue) const
{
	if (InValue.IsSet())
	{
		const int64 Value = InValue.GetValue().Int64;
		if (Value == int64(uint32(~0)))
		{
			return LOCTEXT("AsUInt32InfiniteNumber_Inf", "∞");
		}
		return FText::AsNumber(Value);
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FInt64ValueFormatterAsUInt32InfiniteNumber::FormatValueForTooltip(const TOptional<FTableCellValue>& InValue) const
{
	if (InValue.IsSet())
	{
		const int64 Value = InValue.GetValue().Int64;
		if (Value == int64(uint32(~0)))
		{
			return FText::Format(LOCTEXT("AsUInt32InfiniteNumber_Inf_Fmt", "{0} (∞)"), FText::AsNumber(Value));
		}
		return FText::AsNumber(Value);
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FInt64ValueFormatterAsMemory::FormatValue(const TOptional<FTableCellValue>& InValue) const
{
	if (InValue.IsSet())
	{
		const int64 Value = InValue.GetValue().Int64;
		if (Value > 0)
		{
			return FText::AsMemory(Value, &FormattingOptions);
		}
		else if (Value == 0)
		{
			return LOCTEXT("AsMemory_ZeroValue", "0");
		}
		else // Value < 0
		{
			return FText::Format(LOCTEXT("AsMemory_NegativeValue_Fmt1", "-{0}"), FText::AsMemory(-Value, &FormattingOptions));
		}
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FInt64ValueFormatterAsMemory::FormatValueForTooltip(const TOptional<FTableCellValue>& InValue) const
{
	return InValue.IsSet() ? FormatForTooltip(InValue.GetValue().Int64) : FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FInt64ValueFormatterAsMemory::FormatForTooltip(int64 Value)
{
	if (Value > 0)
	{
		if (Value < 1024)
		{
			if (Value == 1)
			{
				return LOCTEXT("AsMemory_Tooltip_1byte", "1 byte");
			}
			else
			{
				return FText::Format(LOCTEXT("AsMemory_PositiveValue_TooltipFmt1", "{0} bytes"), FText::AsNumber(Value));
			}
		}
		else
		{
			FNumberFormattingOptions FormattingOptions;
			FormattingOptions.MaximumFractionalDigits = 2;
			return FText::Format(LOCTEXT("AsMemory_PositiveValue_TooltipFmt2", "{0} bytes ({1})"), FText::AsNumber(Value), FText::AsMemory(Value, &FormattingOptions));
		}
	}

	if (Value == 0)
	{
		return LOCTEXT("AsMemory_ZeroValue", "0");
	}

	// Value < 0
	{
		if (-Value < 1024)
		{
			if (Value == -1)
			{
				return LOCTEXT("AsMemory_Tooltip_minus1byte", "-1 byte");
			}
			else
			{
				return FText::Format(LOCTEXT("AsMemory_NegativeValue_TooltipFmt1", "-{0} bytes"), FText::AsNumber(-Value));
			}
		}
		else
		{
			FNumberFormattingOptions FormattingOptions;
			FormattingOptions.MaximumFractionalDigits = 2;
			return FText::Format(LOCTEXT("AsMemory_NegativeValue_TooltipFmt2", "-{0} bytes (-{1})"), FText::AsNumber(-Value), FText::AsMemory(-Value, &FormattingOptions));
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FFloatValueFormatterAsNumber::FormatValue(const TOptional<FTableCellValue>& InValue) const
{
	if (InValue.IsSet())
	{
		const float Value = InValue.GetValue().Float;
		if (std::isnan(Value))
		{
			return FText::FromString(TEXT("NaN"));
		}
		else if (Value == 0.0f)
		{
			return FText::FromString(TEXT("0"));
		}
		else
		{
			return FText::FromString(FString::Printf(TEXT("%f"), Value));
		}
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FFloatValueFormatterAsTimeAuto::FormatValue(const TOptional<FTableCellValue>& InValue) const
{
	if (InValue.IsSet())
	{
		const float Value = InValue.GetValue().Float;
		return FText::FromString(FormatTimeAuto(static_cast<double>(Value)));
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FFloatValueFormatterAsTimeAuto::FormatValueForTooltip(const TOptional<FTableCellValue>& InValue) const
{
	if (InValue.IsSet())
	{
		const float Value = InValue.GetValue().Float;
		if (Value == 0.0f)
		{
			return FText::FromString(TEXT("0"));
		}
		else
		{
			return FText::FromString(FString::Printf(TEXT("%f (%s)"), Value, *FormatTimeAuto(static_cast<double>(Value))));
		}
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FDoubleValueFormatterAsNumber::FormatValue(const TOptional<FTableCellValue>& InValue) const
{
	if (InValue.IsSet())
	{
		const double Value = InValue.GetValue().Double;
		if (std::isnan(Value))
		{
			return FText::FromString(TEXT("NaN"));
		}
		else if (Value == 0.0)
		{
			return FText::FromString(TEXT("0"));
		}
		else
		{
			return FText::FromString(FString::Printf(TEXT("%f"), Value));
		}
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FDoubleValueFormatterAsCustomNumber::FormatValue(const TOptional<FTableCellValue>& InValue) const
{
	if (InValue.IsSet())
	{
		const double Value = InValue.GetValue().Double;
		if (std::isnan(Value))
		{
			return FText::FromString(TEXT("NaN"));
		}
		else if (Value == 0.0)
		{
			return FText::FromString(TEXT("0"));
		}
		else
		{
			return FText::AsNumber(Value, &FormattingOptions);
		}
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FDoubleValueFormatterAsPercent::FormatValue(const TOptional<FTableCellValue>& InValue) const
{
	if (InValue.IsSet())
	{
		const double Value = InValue.GetValue().Double;
		if (std::isnan(Value))
		{
			return FText::FromString(TEXT("NaN"));
		}
		else if (Value == 0.0)
		{
			return FText::FromString(TEXT("0%"));
		}
		else if (Value == 1.0)
		{
			return FText::FromString(TEXT("100%"));
		}
		else
		{
			return FText::AsPercent(Value, &FormattingOptions);
		}
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FDoubleValueFormatterAsTimeAuto::FormatValue(const TOptional<FTableCellValue>& InValue) const
{
	if (InValue.IsSet())
	{
		const double Value = InValue.GetValue().Double;
		return FText::FromString(FormatTimeAuto(Value));
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FDoubleValueFormatterAsTimeAuto::FormatValueForTooltip(const TOptional<FTableCellValue>& InValue) const
{
	if (InValue.IsSet())
	{
		const double Value = InValue.GetValue().Double;
		if (Value == 0.0)
		{
			return FText::FromString(TEXT("0"));
		}
		else
		{
			return FText::FromString(FString::Printf(TEXT("%f (%s)"), Value, *FormatTimeAuto(Value)));
		}
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FDoubleValueFormatterAsTimeMs::FormatValue(const TOptional<FTableCellValue>& InValue) const
{
	if (InValue.IsSet())
	{
		const double Value = InValue.GetValue().Double;
		return FText::FromString(FormatTimeMs(Value));
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FDoubleValueFormatterAsTimeMs::FormatValueForTooltip(const TOptional<FTableCellValue>& InValue) const
{
	if (InValue.IsSet())
	{
		const double Value = InValue.GetValue().Double;
		if (Value == 0.0)
		{
			return FText::FromString(TEXT("0"));
		}
		else
		{
			return FText::FromString(FString::Printf(TEXT("%f (%s)"), Value, *FormatTimeMs(Value)));
		}
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FCStringValueFormatterAsText::FormatValue(const TOptional<FTableCellValue>& InValue) const
{
	if (InValue.IsSet())
	{
		const TCHAR* Value = InValue.GetValue().CString;
		if (Value)
		{
			return FText::FromString(Value);
		}
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
