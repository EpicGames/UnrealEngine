// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Table/ViewModels/TableCellValueGetter.h"

#include "InsightsCore/Table/ViewModels/BaseTreeNode.h"

#define LOCTEXT_NAMESPACE "UE::Insights::FTableCellValueGetter"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

const TOptional<FTableCellValue> FNameValueGetter::GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
{
	return TOptional<FTableCellValue>(FTableCellValue(Node.GetDisplayName()));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TOptional<FTableCellValue> FDisplayNameValueGetter::GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
{
	return TOptional<FTableCellValue>(FTableCellValue(Node.GetDisplayName()));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
