// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"

#include "InsightsCore/Table/ViewModels/BaseTreeNode.h"

#define LOCTEXT_NAMESPACE "UE::Insights::FTableCellValueSorter"

// Default pre-sorting (group nodes sorts above leaf nodes)
#define INSIGHTS_DEFAULT_PRESORTING_NODES(A, B) \
	{ \
		if (ShouldCancelSort()) \
		{ \
			return CancelSort(); \
		} \
		if (ShouldSortGroupsFirst() && A->IsGroup() != B->IsGroup()) \
		{ \
			return A->IsGroup(); \
		} \
	}

// Default sorting
#define INSIGHTS_DEFAULT_SORTING_NODES(A, B) return A->GetDefaultSortOrder() < B->GetDefaultSortOrder();

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTableCellValueSorter
////////////////////////////////////////////////////////////////////////////////////////////////////

FTableCellValueSorter::FTableCellValueSorter(const FName InName, const FText& InShortName, const FText& InTitleName, const FText& InDescription, TSharedRef<FTableColumn> InColumnRef)
	: Name(InName)
	, ShortName(InShortName)
	, TitleName(InTitleName)
	, Description(InDescription)
	, ColumnRef(InColumnRef)
	, AscendingIcon(nullptr)
	, DescendingIcon(nullptr)
	, AscendingCompareDelegate(nullptr)
	, DescendingCompareDelegate(nullptr)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTableCellValueSorter::Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const
{
	if (SortMode == ESortMode::Ascending)
	{
		if (AscendingCompareDelegate)
		{
			NodesToSort.Sort(AscendingCompareDelegate);
		}
	}
	else
	{
		if (DescendingCompareDelegate)
		{
			NodesToSort.Sort(DescendingCompareDelegate);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTableCellValueSorter::CancelSort() const
{
#if PLATFORM_EXCEPTIONS_DISABLED
	return true;
#else
	throw "Cancelling sort";
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTableCellValueSorter::ShouldCancelSort() const
{
	return this->AsyncOperationProgress && this->AsyncOperationProgress->ShouldCancelAsyncOp();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FBaseTableColumnSorter
////////////////////////////////////////////////////////////////////////////////////////////////////

FBaseTableColumnSorter::FBaseTableColumnSorter(TSharedRef<FTableColumn> InColumnRef)
	: FTableCellValueSorter(
		InColumnRef->GetId(),
		FText::Format(LOCTEXT("Sorter_ColumnValue_Name", "By {0}"), InColumnRef->GetShortName()),
		FText::Format(LOCTEXT("Sorter_ColumnValue_Title", "Sort By {0}"), InColumnRef->GetTitleName()),
		FText::Format(LOCTEXT("Sorter_ColumnValue_Desc", "Sort by {0}."), InColumnRef->GetShortName()),
		InColumnRef)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FSorterByName
////////////////////////////////////////////////////////////////////////////////////////////////////

FSorterByName::FSorterByName(TSharedRef<FTableColumn> InColumnRef)
	: FTableCellValueSorter(
		FName(TEXT("ByName")),
		LOCTEXT("Sorter_ByName_Name", "By Name"),
		LOCTEXT("Sorter_ByName_Title", "Sort By Name"),
		LOCTEXT("Sorter_ByName_Desc", "Sort alphabetically by name."),
		InColumnRef)
{
	AscendingCompareDelegate = [this](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
	{
		INSIGHTS_DEFAULT_PRESORTING_NODES(A, B)

		// Sort by name (ascending).
		return A->GetDisplayName().CompareTo(B->GetDisplayName()) < 0;
	};

	DescendingCompareDelegate = [this](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
	{
		INSIGHTS_DEFAULT_PRESORTING_NODES(A, B)

		// Sort by name (descending).
		return B->GetDisplayName().CompareTo(A->GetDisplayName()) < 0;
	};
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FSorterByTypeName
////////////////////////////////////////////////////////////////////////////////////////////////////

FSorterByTypeName::FSorterByTypeName(TSharedRef<FTableColumn> InColumnRef)
	: FTableCellValueSorter(
		FName(TEXT("ByTypeName")),
		LOCTEXT("Sorter_ByTypeName_Name", "By Type Name"),
		LOCTEXT("Sorter_ByTypeName_Title", "Sort By Type Name"),
		LOCTEXT("Sorter_ByTypeName_Desc", "Sort by type name."),
		InColumnRef)
{
	AscendingCompareDelegate = [this](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
	{
		INSIGHTS_DEFAULT_PRESORTING_NODES(A, B)

		const FName& TypeA = A->GetTypeName();
		const FName& TypeB = B->GetTypeName();

		if (TypeA == TypeB)
		{
			INSIGHTS_DEFAULT_SORTING_NODES(A, B)
		}
		else
		{
			// Sort by type name (ascending).
			return TypeA.FastLess(TypeB); // TypeA < TypeB;
		}
	};

	DescendingCompareDelegate = [this](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
	{
		INSIGHTS_DEFAULT_PRESORTING_NODES(A, B)

		const FName& TypeA = A->GetTypeName();
		const FName& TypeB = B->GetTypeName();

		if (TypeA == TypeB)
		{
			INSIGHTS_DEFAULT_SORTING_NODES(A, B)
		}
		else
		{
			// Sort by type name (descending).
			return TypeB.FastLess(TypeA); // TypeA > TypeB;
		}
	};
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FSorterByBoolValue
////////////////////////////////////////////////////////////////////////////////////////////////////

FSorterByBoolValue::FSorterByBoolValue(TSharedRef<FTableColumn> InColumnRef)
	: FBaseTableColumnSorter(InColumnRef)
{
	ensure(InColumnRef->GetDataType() == ETableCellDataType::Bool);

	AscendingCompareDelegate = [&Column = *InColumnRef, this](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
	{
		INSIGHTS_DEFAULT_PRESORTING_NODES(A, B)

		const TOptional<FTableCellValue> OptionalValueA = Column.GetValue(*A);
		const TOptional<FTableCellValue> OptionalValueB = Column.GetValue(*B);

		const bool ValueA = OptionalValueA.IsSet() ? OptionalValueA.GetValue().Bool : false;
		const bool ValueB = OptionalValueB.IsSet() ? OptionalValueB.GetValue().Bool : false;

		if (ValueA == ValueB)
		{
			INSIGHTS_DEFAULT_SORTING_NODES(A, B)
		}
		else
		{
			// Sort by value (ascending).
			return ValueA < ValueB;
		}
	};

	DescendingCompareDelegate = [&Column = *InColumnRef, this](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
	{
		INSIGHTS_DEFAULT_PRESORTING_NODES(A, B)

		const TOptional<FTableCellValue> OptionalValueA = Column.GetValue(*A);
		const TOptional<FTableCellValue> OptionalValueB = Column.GetValue(*B);

		const bool ValueA = OptionalValueA.IsSet() ? OptionalValueA.GetValue().Bool : false;
		const bool ValueB = OptionalValueB.IsSet() ? OptionalValueB.GetValue().Bool : false;

		if (ValueA == ValueB)
		{
			INSIGHTS_DEFAULT_SORTING_NODES(A, B)
		}
		else
		{
			// Sort by value (descending).
			return ValueB < ValueA;
		}
	};
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FSorterByInt64Value
////////////////////////////////////////////////////////////////////////////////////////////////////

FSorterByInt64Value::FSorterByInt64Value(TSharedRef<FTableColumn> InColumnRef)
	: FBaseTableColumnSorter(InColumnRef)
{
	ensure(InColumnRef->GetDataType() == ETableCellDataType::Int64);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSorterByInt64Value::Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const
{
	// Note: This implementation is faster than using delegates.
	//       It caches the values before sorting, in this way it minimizes the calls to Column.GetValue().

	const FTableColumn& Column = *ColumnRef;

	const int32 NumElements = NodesToSort.Num();

	struct FSortElement
	{
		const FBaseTreeNodePtr NodePtr;
		int64 Value;
	};
	TArray<FSortElement> ElementsToSort;

	// Cache value for each node.
	ElementsToSort.Reset(NumElements);
	for (const FBaseTreeNodePtr& NodePtr : NodesToSort)
	{
		if (ShouldCancelSort())
		{
			return;
		}

		const TOptional<FTableCellValue> OptionalValue = Column.GetValue(*NodePtr);
		const int64 Value = OptionalValue.IsSet() ? OptionalValue.GetValue().Int64 : 0;
		ElementsToSort.Add({ NodePtr, Value });
	}
	ensure(ElementsToSort.Num() == NumElements);

	if (SortMode == ESortMode::Ascending)
	{
		ElementsToSort.Sort([this](const FSortElement& A, const FSortElement& B) -> bool
		{
			INSIGHTS_DEFAULT_PRESORTING_NODES(A.NodePtr, B.NodePtr)

			if (A.Value == B.Value)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A.NodePtr, B.NodePtr)
			}
			else
			{
				// Sort by value (ascending).
				return A.Value < B.Value;
			}
		});
	}
	else // if (SortMode == ESortMode::Descending)
	{
		ElementsToSort.Sort([this](const FSortElement& A, const FSortElement& B) -> bool
		{
			INSIGHTS_DEFAULT_PRESORTING_NODES(A.NodePtr, B.NodePtr)

			if (A.Value == B.Value)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A.NodePtr, B.NodePtr)
			}
			else
			{
				// Sort by value (descending).
				return B.Value < A.Value;
			}
		});
	}

	for (int32 Index = 0; Index < NumElements; ++Index)
	{
		NodesToSort[Index] = ElementsToSort[Index].NodePtr;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FSorterByFloatValue
////////////////////////////////////////////////////////////////////////////////////////////////////

FSorterByFloatValue::FSorterByFloatValue(TSharedRef<FTableColumn> InColumnRef)
	: FBaseTableColumnSorter(InColumnRef)
{
	ensure(InColumnRef->GetDataType() == ETableCellDataType::Float);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSorterByFloatValue::Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const
{
	// Note: This implementation is faster than using delegates.
	//       It caches the values before sorting, in this way it minimizes the calls to Column.GetValue().

	const FTableColumn& Column = *ColumnRef;

	const int32 NumElements = NodesToSort.Num();

	struct FSortElement
	{
		const FBaseTreeNodePtr NodePtr;
		float Value;
	};
	TArray<FSortElement> ElementsToSort;

	// Cache value for each node.
	ElementsToSort.Reset(NumElements);
	for (const FBaseTreeNodePtr& NodePtr : NodesToSort)
	{
		if (ShouldCancelSort())
		{
			return;
		}

		const TOptional<FTableCellValue> OptionalValue = Column.GetValue(*NodePtr);
		const float Value = OptionalValue.IsSet() ? OptionalValue.GetValue().Float : 0.0f;
		ElementsToSort.Add({ NodePtr, Value });
	}
	ensure(ElementsToSort.Num() == NumElements);

	if (SortMode == ESortMode::Ascending)
	{
		ElementsToSort.Sort([this](const FSortElement& A, const FSortElement& B) -> bool
		{
			INSIGHTS_DEFAULT_PRESORTING_NODES(A.NodePtr, B.NodePtr)

			if (A.Value == B.Value)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A.NodePtr, B.NodePtr)
			}
			else
			{
				// Sort by value (ascending).
				return A.Value < B.Value;
			}
		});
	}
	else // if (SortMode == ESortMode::Descending)
	{
		ElementsToSort.Sort([this](const FSortElement& A, const FSortElement& B) -> bool
		{
			INSIGHTS_DEFAULT_PRESORTING_NODES(A.NodePtr, B.NodePtr)

			if (A.Value == B.Value)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A.NodePtr, B.NodePtr)
			}
			else
			{
				// Sort by value (descending).
				return B.Value < A.Value;
			}
		});
	}

	for (int32 Index = 0; Index < NumElements; ++Index)
	{
		NodesToSort[Index] = ElementsToSort[Index].NodePtr;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FSorterByDoubleValue
////////////////////////////////////////////////////////////////////////////////////////////////////

FSorterByDoubleValue::FSorterByDoubleValue(TSharedRef<FTableColumn> InColumnRef)
	: FBaseTableColumnSorter(InColumnRef)
{
	ensure(InColumnRef->GetDataType() == ETableCellDataType::Double);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSorterByDoubleValue::Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const
{
	// Note: This implementation is faster than using delegates.
	//       It caches the values before sorting, in this way it minimizes the calls to Column.GetValue().

	const FTableColumn& Column = *ColumnRef;

	const int32 NumElements = NodesToSort.Num();

	struct FSortElement
	{
		const FBaseTreeNodePtr NodePtr;
		double Value;
	};
	TArray<FSortElement> ElementsToSort;

	// Cache value for each node.
	ElementsToSort.Reset(NumElements);
	for (const FBaseTreeNodePtr& NodePtr : NodesToSort)
	{
		if (ShouldCancelSort())
		{
			return;
		}

		const TOptional<FTableCellValue> OptionalValue = Column.GetValue(*NodePtr);
		const double Value = OptionalValue.IsSet() ? OptionalValue.GetValue().Double : 0.0;
		ElementsToSort.Add({ NodePtr, Value });
	}
	ensure(ElementsToSort.Num() == NumElements);

	if (SortMode == ESortMode::Ascending)
	{
		ElementsToSort.Sort([this](const FSortElement& A, const FSortElement& B) -> bool
		{
			INSIGHTS_DEFAULT_PRESORTING_NODES(A.NodePtr, B.NodePtr)

			if (A.Value == B.Value)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A.NodePtr, B.NodePtr)
			}
			else
			{
				// Sort by value (ascending).
				return A.Value < B.Value;
			}
		});
	}
	else // if (SortMode == ESortMode::Descending)
	{
		ElementsToSort.Sort([this](const FSortElement& A, const FSortElement& B) -> bool
		{
			INSIGHTS_DEFAULT_PRESORTING_NODES(A.NodePtr, B.NodePtr)

			if (A.Value == B.Value)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A.NodePtr, B.NodePtr)
			}
			else
			{
				// Sort by value (descending).
				return B.Value < A.Value;
			}
		});
	}

	for (int32 Index = 0; Index < NumElements; ++Index)
	{
		NodesToSort[Index] = ElementsToSort[Index].NodePtr;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FSorterByCStringValue
////////////////////////////////////////////////////////////////////////////////////////////////////

FSorterByCStringValue::FSorterByCStringValue(TSharedRef<FTableColumn> InColumnRef)
	: FBaseTableColumnSorter(InColumnRef)
{
	ensure(InColumnRef->GetDataType() == ETableCellDataType::CString);

	AscendingCompareDelegate = [&Column = *InColumnRef, this](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
	{
		INSIGHTS_DEFAULT_PRESORTING_NODES(A, B)

		const TOptional<FTableCellValue> OptionalValueA = Column.GetValue(*A);
		const TOptional<FTableCellValue> OptionalValueB = Column.GetValue(*B);

		const TCHAR* ValueA = OptionalValueA.IsSet() ? OptionalValueA.GetValue().CString : nullptr;
		const TCHAR* ValueB = OptionalValueB.IsSet() ? OptionalValueB.GetValue().CString : nullptr;

		// If any value is nullptr
		if (!ValueA || !ValueB)
		{
			if (!ValueA && !ValueB)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			return ValueA < ValueB;
		}

		const int32 Compare = FCString::Stricmp(ValueA, ValueB);
		if (Compare == 0)
		{
			INSIGHTS_DEFAULT_SORTING_NODES(A, B)
		}
		else
		{
			// Sort by value (ascending).
			return Compare < 0;
		}
	};

	DescendingCompareDelegate = [&Column = *InColumnRef, this](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
	{
		INSIGHTS_DEFAULT_PRESORTING_NODES(A, B)

		const TOptional<FTableCellValue> OptionalValueA = Column.GetValue(*A);
		const TOptional<FTableCellValue> OptionalValueB = Column.GetValue(*B);

		const TCHAR* ValueA = OptionalValueA.IsSet() ? OptionalValueA.GetValue().CString : nullptr;
		const TCHAR* ValueB = OptionalValueB.IsSet() ? OptionalValueB.GetValue().CString : nullptr;

		// If any value is nullptr
		if (!ValueA || !ValueB)
		{
			if (!ValueA && !ValueB)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			return ValueA > ValueB;
		}

		const int32 Compare = FCString::Stricmp(ValueA, ValueB);
		if (Compare == 0)
		{
			INSIGHTS_DEFAULT_SORTING_NODES(A, B)
		}
		else
		{
			// Sort by value (descending).
			return Compare > 0;
		}
	};
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FSorterByTextValue
////////////////////////////////////////////////////////////////////////////////////////////////////

FSorterByTextValue::FSorterByTextValue(TSharedRef<FTableColumn> InColumnRef)
	: FBaseTableColumnSorter(InColumnRef)
{
	ensure(InColumnRef->GetDataType() == ETableCellDataType::Text);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSorterByTextValue::Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const
{
	// Note: This implementation is faster than using delegates.
	//       It caches the values before sorting, in this way it minimizes the calls to Column.GetValue().

	const FTableColumn& Column = *ColumnRef;

	const int32 NumElements = NodesToSort.Num();

	struct FSortElement
	{
		const FBaseTreeNodePtr NodePtr;
		FSetElementId ValueId;
	};
	TArray<FSortElement> ElementsToSort;

	TSet<FString> UniqueValues;

	// Cache value for each node.
	ElementsToSort.Reset(NumElements);
	for (const FBaseTreeNodePtr& NodePtr : NodesToSort)
	{
		if (ShouldCancelSort())
		{
			return;
		}

		FSetElementId ValueId;
		TOptional<FTableCellValue> Value = Column.GetValue(*NodePtr);
		if (Value.IsSet())
		{
			ValueId = UniqueValues.Add(Value.GetValue().GetText().ToString());
		}
		ElementsToSort.Add({ NodePtr, ValueId });
	}
	ensure(ElementsToSort.Num() == NumElements);

	if (SortMode == ESortMode::Ascending)
	{
		ElementsToSort.Sort([&UniqueValues, this](const FSortElement& A, const FSortElement& B) -> bool
		{
			INSIGHTS_DEFAULT_PRESORTING_NODES(A.NodePtr, B.NodePtr)

			if (A.ValueId == B.ValueId)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A.NodePtr, B.NodePtr)
			}

			if (!A.ValueId.IsValidId())
			{
				return true;
			}

			if (!B.ValueId.IsValidId())
			{
				return false;
			}

			const FString& ValueA = UniqueValues[A.ValueId];
			const FString& ValueB = UniqueValues[B.ValueId];

			// Sort by value (ascending).
			return ValueA.Compare(ValueB, ESearchCase::IgnoreCase) <= 0;
		});
	}
	else // if (SortMode == ESortMode::Descending)
	{
		ElementsToSort.Sort([&UniqueValues, this](const FSortElement& A, const FSortElement& B) -> bool
		{
			INSIGHTS_DEFAULT_PRESORTING_NODES(A.NodePtr, B.NodePtr)

			if (A.ValueId == B.ValueId)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A.NodePtr, B.NodePtr)
			}

			if (!A.ValueId.IsValidId())
			{
				return true;
			}

			if (!B.ValueId.IsValidId())
			{
				return false;
			}

			const FString& ValueA = UniqueValues[A.ValueId];
			const FString& ValueB = UniqueValues[B.ValueId];

			// Sort by value (descending).
			return ValueA.Compare(ValueB, ESearchCase::IgnoreCase) >= 0;
		});
	}

	for (int32 Index = 0; Index < NumElements; ++Index)
	{
		NodesToSort[Index] = ElementsToSort[Index].NodePtr;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FSorterByTextValueWithId
////////////////////////////////////////////////////////////////////////////////////////////////////

FSorterByTextValueWithId::FSorterByTextValueWithId(TSharedRef<FTableColumn> InColumnRef)
	: FBaseTableColumnSorter(InColumnRef)
{
	ensure(InColumnRef->GetDataType() == ETableCellDataType::Text);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSorterByTextValueWithId::Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const
{
	// Note: This implementation is faster than using delegates.
	//       It caches the values before sorting, in this way it minimizes the calls to Column.GetValue().

	const FTableColumn& Column = *ColumnRef;

	const int32 NumElements = NodesToSort.Num();

	struct FSortElement
	{
		const FBaseTreeNodePtr NodePtr;
		int32 ValueIndex;
	};
	TArray<FSortElement> ElementsToSort;

	TArray<FString> UniqueValues;
	TMap<uint64, int32> UniqueValuesMap; // Id --> Index in UniqueValues

	// Cache value for each node.
	ElementsToSort.Reset(NumElements);
	for (const FBaseTreeNodePtr& NodePtr : NodesToSort)
	{
		if (ShouldCancelSort())
		{
			return;
		}

		int32 ValueIndex = -1;
		uint64 ValueId = Column.GetValueId(*NodePtr);
		const int32* IndexPtr = UniqueValuesMap.Find(ValueId);
		if (IndexPtr)
		{
			ValueIndex = *IndexPtr;
		}
		else
		{
			TOptional<FTableCellValue> Value = Column.GetValue(*NodePtr);
			if (Value.IsSet())
			{
				ValueIndex = UniqueValues.Num();
				UniqueValues.Add(Value.GetValue().GetText().ToString());
			}
			UniqueValuesMap.Add(ValueId, ValueIndex);
		}
		ElementsToSort.Add({ NodePtr, ValueIndex });
	}
	ensure(ElementsToSort.Num() == NumElements);

	if (SortMode == ESortMode::Ascending)
	{
		ElementsToSort.Sort([&UniqueValues, this](const FSortElement& A, const FSortElement& B) -> bool
		{
			INSIGHTS_DEFAULT_PRESORTING_NODES(A.NodePtr, B.NodePtr)

			if (A.ValueIndex == B.ValueIndex)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A.NodePtr, B.NodePtr)
			}

			if (A.ValueIndex < 0)
			{
				return true;
			}

			if (B.ValueIndex < 0)
			{
				return false;
			}

			const FString& ValueA = UniqueValues[A.ValueIndex];
			const FString& ValueB = UniqueValues[B.ValueIndex];

			// Sort by value (ascending).
			return ValueA.Compare(ValueB, ESearchCase::IgnoreCase) <= 0;
		});
	}
	else // if (SortMode == ESortMode::Descending)
	{
		ElementsToSort.Sort([&UniqueValues, this](const FSortElement& A, const FSortElement& B) -> bool
		{
			INSIGHTS_DEFAULT_PRESORTING_NODES(A.NodePtr, B.NodePtr)

			if (A.ValueIndex == B.ValueIndex)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A.NodePtr, B.NodePtr)
			}

			if (A.ValueIndex < 0)
			{
				return true;
			}

			if (B.ValueIndex < 0)
			{
				return false;
			}

			const FString& ValueA = UniqueValues[A.ValueIndex];
			const FString& ValueB = UniqueValues[B.ValueIndex];

			// Sort by value (descending).
			return ValueA.Compare(ValueB, ESearchCase::IgnoreCase) >= 0;
		});
	}

	for (int32 Index = 0; Index < NumElements; ++Index)
	{
		NodesToSort[Index] = ElementsToSort[Index].NodePtr;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef INSIGHTS_DEFAULT_SORTING_NODES
#undef INSIGHTS_DEFAULT_PRESORTING_NODES
#undef LOCTEXT_NAMESPACE
