// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Table/ViewModels/TableColumn.h"

#include "InsightsCore/Table/ViewModels/TableCellValueFormatter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueGetter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"

#include <limits>
#include <cmath>

#define LOCTEXT_NAMESPACE "UE::Insights::FTableColumn"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<ITableCellValueGetter> FTableColumn::GetDefaultValueGetter()
{
	return MakeShared<FTableCellValueGetter>();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TOptional<FTableCellValue> FTableColumn::GetValue(const FBaseTreeNode& InNode) const
{
	return ValueGetter->GetValue(*this, InNode);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint64 FTableColumn::GetValueId(const FBaseTreeNode& InNode) const
{
	return ValueGetter->GetValueId(*this, InNode);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<ITableCellValueFormatter> FTableColumn::GetDefaultValueFormatter()
{
	return MakeShared<FTableCellValueFormatter>();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FTableColumn::GetValueAsText(const FBaseTreeNode& InNode) const
{
	return ValueFormatter->FormatValue(*this, InNode);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FTableColumn::GetValueAsTooltipText(const FBaseTreeNode& InNode) const
{
	return ValueFormatter->FormatValueForTooltip(*this, InNode);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FTableColumn::GetValueAsGroupingText(const FBaseTreeNode& InNode) const
{
	return ValueFormatter->FormatValueForGrouping(*this, InNode);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FTableColumn::CopyValue(const FBaseTreeNode& InNode) const
{
	return ValueFormatter->CopyValue(*this, InNode);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FTableColumn::CopyTooltip(const FBaseTreeNode& InNode) const
{
	return ValueFormatter->CopyTooltip(*this, InNode);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

inline void TrimEndingZero(FString& Str)
{
	const TCHAR* Start = *Str;
	const TCHAR* End = Start + Str.Len();
	const TCHAR* Data = End;
	while (Data != Start)
	{
		--Data;
		if (*Data != TEXT('0'))
		{
			break;
		}
	}
	if (Data != End)
	{
		if (*Data == TEXT('.'))
		{
			--Data;
		}
		Str.LeftInline(static_cast<int32>(Data - Start + 1));
	}
}

FString FTableColumn::GetValueAsSerializableString(const FBaseTreeNode& InNode) const
{
	const TOptional<FTableCellValue> Value = GetValue(InNode);
	if (!Value.IsSet())
	{
		return FString();
	}
	switch (Value->DataType)
	{
		case ETableCellDataType::Double:
		{
			const double DblValue = Value->AsDouble();
			if (DblValue == 0.0)
			{
				return FString("0");
			}
			else if (DblValue == +DBL_MAX || DblValue == +std::numeric_limits<double>::infinity())
			{
				return FString("+inf");
			}
			else if (DblValue == -DBL_MAX || DblValue == -std::numeric_limits<double>::infinity())
			{
				return FString("-inf");
			}
			else if (std::isnan(DblValue))
			{
				return FString("nan");
			}
			else
			{
				FString Str = FString::Printf(TEXT("%.9f"), DblValue);
				TrimEndingZero(Str);
				return Str;
			}
		}
		case ETableCellDataType::Float:
		{
			const float FltValue = Value->AsFloat();
			if (FltValue == 0.0f)
			{
				return FString("0");
			}
			else if (FltValue == +FLT_MAX || FltValue == +std::numeric_limits<float>::infinity())
			{
				return FString("+inf");
			}
			else if (FltValue == -FLT_MAX || FltValue == -std::numeric_limits<float>::infinity())
			{
				return FString("-inf");
			}
			else if (std::isnan(FltValue))
			{
				return FString("nan");
			}
			else
			{
				FString Str = FString::Printf(TEXT("%.6f"), FltValue);
				TrimEndingZero(Str);
				return Str;
			}
		}
		case ETableCellDataType::Int64:
			return FString::Printf(TEXT("%lli"), Value->AsInt64());
		case ETableCellDataType::Bool:
			return Value->AsBool() ? FString("true") : FString("false");
		case ETableCellDataType::CString:
			return Value->AsString();
		case ETableCellDataType::Text:
			return Value->AsText().ToString();
		default:
			return GetValueAsText(InNode).ToString();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
