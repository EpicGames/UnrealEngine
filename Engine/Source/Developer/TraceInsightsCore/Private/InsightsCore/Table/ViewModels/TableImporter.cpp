// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Table/ViewModels/TableImporter.h"

#include "DesktopPlatformModule.h"
//include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
//#include "Framework/Docking/WorkspaceItem.h"
#include "Logging/MessageLog.h"
#include "Misc/FileHelper.h"
#include "SlateOptMacros.h"
#include "Widgets/Docking/SDockTab.h"

// TraceServices
#include "TraceServices/Model/Threads.h"

// TraceInsightsCore
#include "InsightsCore/Common/InsightsCoreStyle.h"
#include "InsightsCore/Common/Log.h"
#include "InsightsCore/Common/Stopwatch.h"
#include "InsightsCore/Table/ViewModels/UntypedTable.h"
#include "InsightsCore/Table/Widgets/SUntypedDiffTableTreeView.h"
#include "InsightsCore/Table/Widgets/SUntypedTableTreeView.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TableImporter"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

FTableImporter::FTableImporter(FName InLogListingName)
	: LogListingName(InLogListingName)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTableImporter::~FTableImporter()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTableImporter::StartImportProcess()
{
	TArray<FString> Filenames;
	bool bDialogResult = false;

	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		const FString DefaultPath = FPaths::ProjectSavedDir();
		const FString DefaultFile = TEXT("");
		bDialogResult = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			LOCTEXT("ImportTableTitle", "Import Table").ToString(),
			DefaultPath,
			DefaultFile,
			TEXT("Comma-Separated Values (*.csv)|*.csv|Tab-Separated Values (*.tsv)|*.tsv"),
			EFileDialogFlags::None,
			Filenames
		);
	}

	if (!bDialogResult)
	{
		return;
	}

	if (Filenames.Num() != 1)
	{
		FMessageLog ReportMessageLog(LogListingName);
		ReportMessageLog.Error(LOCTEXT("NotSingleFile", "A single file can be imported at a time!"));
		ReportMessageLog.Notify();
		return;
	}

	ImportFile(Filenames[0]);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTableImporter::ImportFile(const FString& Filename)
{
	using namespace TraceServices;

	FName TableId = GetTableID(Filename);
	DisplayImportTable(TableId);

	FTableImportService::ImportTable(Filename,
		TableId,
		[this](TSharedPtr<FTableImportCallbackParams> Params) {
			this->TableImportServiceCallback(Params);
		});
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTableImporter::StartDiffProcess()
{
	TArray<FString> Filenames;
	bool bDialogResult = false;

	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		const FString DefaultPath = FPaths::ProjectSavedDir();
		const FString DefaultFile = TEXT("");
		bDialogResult = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			LOCTEXT("DiffTableTitle", "Choose Tables").ToString(),
			DefaultPath,
			DefaultFile,
			TEXT("Comma-Separated Values (*.csv)|*.csv|Tab-Separated Values (*.tsv)|*.tsv"),
			EFileDialogFlags::Multiple,
			Filenames
		);
	}

	if (!bDialogResult)
	{
		return;
	}

	if (DesktopPlatform && Filenames.Num() <  2)
	{
		const FString DefaultPath = FPaths::ProjectSavedDir();
		const FString DefaultFile = TEXT("");
		bDialogResult = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			LOCTEXT("DiffTableBTitle", "Choose Table B").ToString(),
			DefaultPath,
			DefaultFile,
			TEXT("Comma-Separated Values (*.csv)|*.csv|Tab-Separated Values (*.tsv)|*.tsv"),
			EFileDialogFlags::None,
			Filenames
		);
	}

	if (!bDialogResult)
	{
		return;
	}

	if (Filenames.Num() != 2)
	{
		FMessageLog ReportMessageLog(LogListingName);
		ReportMessageLog.Error(LOCTEXT("NotEnoughFilesSelected", "Exactly 2 files are required to be chosen!"));
		ReportMessageLog.Notify();
		return;
	}

	DiffFiles(Filenames[0], Filenames[1]);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTableImporter::DiffFiles(const FString& FilenameA, const FString& FilenameB)
{
	using namespace TraceServices;

	FName TableViewID = GetTableID(FilenameA+TEXT(",")+FilenameB);
	DisplayDiffTable(TableViewID);

	FTableImportService::ImportTable(FilenameA, TableViewID,
		[this, Name = FilenameA](TSharedPtr<FTableImportCallbackParams> Params)
		{
			ensure(Params.IsValid());

			if (const FOpenTableTabData* TableViewPtr = OpenTablesMap.Find(Params->TableId))
			{
				TSharedPtr<SUntypedDiffTableTreeView> TableView = StaticCastSharedPtr<SUntypedDiffTableTreeView>(TableViewPtr->TableTreeView);

				if (Params->Result == TraceServices::ETableImportResult::ESuccess)
				{
					TableView->UpdateSourceTableA(Name, Params->Table);
				}
				else
				{
					FMessageLog ReportMessageLog(LogListingName);
					ReportMessageLog.AddMessages(Params->Messages);
					ReportMessageLog.Notify();
				}
			}
		});

	FTableImportService::ImportTable(FilenameB, TableViewID,
		[this, Name = FilenameB](TSharedPtr<FTableImportCallbackParams> Params)
		{
			ensure(Params.IsValid());

			if (const FOpenTableTabData* TableViewPtr = OpenTablesMap.Find(Params->TableId))
			{
				TSharedPtr<SUntypedDiffTableTreeView> TableView = StaticCastSharedPtr<SUntypedDiffTableTreeView>(TableViewPtr->TableTreeView);

				if (Params->Result == TraceServices::ETableImportResult::ESuccess)
				{
					TableView->UpdateSourceTableB(Name, Params->Table);
				}
				else
				{
					FMessageLog ReportMessageLog(LogListingName);
					ReportMessageLog.AddMessages(Params->Messages);
					ReportMessageLog.Notify();
				}
			}
		});
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTableImporter::DisplayImportTable(FName TableViewID)
{
	FText ImportTableTreeViewTabDisplayName;
	FString FileName = FPaths::GetCleanFilename(TableViewID.GetPlainNameString());
	if (TableViewID.GetNumber())
	{
		ImportTableTreeViewTabDisplayName = FText::Format(LOCTEXT("TableImportTool.ImportTableTreeViewTabDisplayName", "{0}({1})"),
														  FText::FromString(FileName),
														  FText::AsNumber(TableViewID.GetNumber()));
	}
	else
	{
		ImportTableTreeViewTabDisplayName = FText::FromString(FileName);
	}

	FString TooltipText = FPaths::ConvertRelativePathToFull(TableViewID.GetPlainNameString());
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(TableViewID, FOnSpawnTab::CreateSP(this, &FTableImporter::SpawnTab_TableImportTreeView, TableViewID, ImportTableTreeViewTabDisplayName))
		.SetDisplayName(ImportTableTreeViewTabDisplayName)
		.SetTooltipText(FText::FromString(TooltipText))
		.SetIcon(FSlateIcon(FInsightsCoreStyle::GetStyleSetName(), "Icons.TableTreeView"));

	FGlobalTabmanager::Get()->TryInvokeTab(TableViewID);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTableImporter::DisplayDiffTable(FName TableViewID)
{
	FText ImportTableTreeViewTabDisplayName;
	TArray<FString> Filenames;
	ensure(TableViewID.GetPlainNameString().ParseIntoArray(Filenames, TEXT(",")) == 2);

	const FText FilenameA = FText::FromString(FPaths::GetCleanFilename(Filenames[0]));
	const FText FilenameB = FText::FromString(FPaths::GetCleanFilename(Filenames[1]));

	if (TableViewID.GetNumber())
	{
		ImportTableTreeViewTabDisplayName = FText::Format(
			LOCTEXT("TableImportTool.DiffTablesNumericTreeViewTabDisplayName", "{0} vs {1} ({2})"),
			FilenameA,
			FilenameB,
			FText::AsNumber(TableViewID.GetNumber()));
	}
	else
	{
		ImportTableTreeViewTabDisplayName = FText::Format(
			LOCTEXT("TableImportTool.DiffTablesTableTreeViewTabDisplayName", "{0} vs {1}"),
			FilenameA,
			FilenameB);
	}
	const FText ImportTableTreeViewTabTooltipText = FText::Format(
			LOCTEXT("TableImportTool.DiffTablesTableTreeViewTabTooltip", "{0} vs {1}"),
			FilenameA,
			FilenameB);

	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(TableViewID,
		FOnSpawnTab::CreateSP(this, &FTableImporter::SpawnTab_TableDiffTreeView, TableViewID, ImportTableTreeViewTabDisplayName))
		.SetDisplayName(ImportTableTreeViewTabDisplayName)
		.SetTooltipText(ImportTableTreeViewTabTooltipText)
		.SetIcon(FSlateIcon(FInsightsCoreStyle::GetStyleSetName(), "Icons.TableTreeView"));

	FGlobalTabmanager::Get()->TryInvokeTab(TableViewID);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTableImporter::TableImportServiceCallback(TSharedPtr<TraceServices::FTableImportCallbackParams> Params)
{
	ensure(Params.IsValid());

	const FOpenTableTabData* TableViewPtr = OpenTablesMap.Find(Params->TableId);
	if (TableViewPtr)
	{
		TSharedPtr<SUntypedTableTreeView> TableView = TableViewPtr->TableTreeView;
		TableView->ClearCurrentOperationNameOverride();

		if (Params->Result == TraceServices::ETableImportResult::ESuccess)
		{
			TableView->UpdateSourceTable(Params->Table);
		}
		else
		{
			FMessageLog ReportMessageLog(LogListingName);
			ReportMessageLog.AddMessages(Params->Messages);
			ReportMessageLog.Notify();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> FTableImporter::SpawnTab_TableImportTreeView(const FSpawnTabArgs& Args, FName TableViewID, FText InDisplayName)
{
	TSharedRef<FUntypedTable> UntypedTable = MakeShared<Insights::FUntypedTable>();
	UntypedTable->Reset();
	UntypedTable->SetDisplayName(InDisplayName);

	TSharedPtr<SUntypedTableTreeView> TableTreeView;

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(TableTreeView, SUntypedTableTreeView, UntypedTable)
			.RunInAsyncMode(true)
		];

	TableTreeView->SetCurrentOperationNameOverride(LOCTEXT("TableImportMessage", "Importing Data"));

	FOpenTableTabData OpenTabData;
	OpenTabData.TableTreeView = TableTreeView;
	OpenTabData.Tab = DockTab;

	OpenTablesMap.Add(TableViewID, OpenTabData);
	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateSP(this, &FTableImporter::OnTableImportTreeViewTabClosed));

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> FTableImporter::SpawnTab_TableDiffTreeView(const FSpawnTabArgs& Args, FName TableViewID, FText InDisplayName)
{
	TSharedRef<FUntypedTable> UntypedTable = MakeShared<Insights::FUntypedTable>();
	UntypedTable->Reset();
	UntypedTable->SetDisplayName(InDisplayName);

	TSharedPtr<Insights::SUntypedDiffTableTreeView> TableTreeView;

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(TableTreeView, SUntypedDiffTableTreeView, UntypedTable)
			.RunInAsyncMode(true)
		];

	TableTreeView->SetCurrentOperationNameOverride(LOCTEXT("TableImportMessage", "Importing Data"));

	FOpenTableTabData OpenTabData;
	OpenTabData.TableTreeView = TableTreeView;
	OpenTabData.Tab = DockTab;

	OpenTablesMap.Add(TableViewID, OpenTabData);
	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateSP(this, &FTableImporter::OnTableImportTreeViewTabClosed));

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTableImporter::OnTableImportTreeViewTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	TSharedRef<SUntypedTableTreeView> TableTreeView = StaticCastSharedRef<SUntypedTableTreeView>(TabBeingClosed->GetContent());
	FOpenTableTabData OpenTabData;
	OpenTabData.TableTreeView = TableTreeView;
	OpenTabData.Tab = TabBeingClosed;
	const FName* TableIdPtr = OpenTablesMap.FindKey(OpenTabData);
	check(TableIdPtr);

	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(*TableIdPtr);
	TableTreeView->OnClose();
	OpenTablesMap.Remove(*TableIdPtr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FName FTableImporter::GetTableID(const FString& Path)
{
	FName TableId(Path);
	while (OpenTablesMap.Contains(TableId))
	{
		TableId.SetNumber(TableId.GetNumber() + 1);
	}

	return TableId;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTableImporter::CloseAllOpenTabs()
{
	while (!OpenTablesMap.IsEmpty())
	{
		TSharedPtr<SDockTab> Tab = OpenTablesMap.CreateConstIterator().Value().Tab;
		Tab->RequestCloseTab();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
