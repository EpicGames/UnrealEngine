// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Table/ViewModels/TableTreeNode.h"

#define LOCTEXT_NAMESPACE "UE::Insights::FTableTreeNode"

namespace UE::Insights
{

INSIGHTS_IMPLEMENT_RTTI(FTableTreeNode)
INSIGHTS_IMPLEMENT_RTTI(FCustomTableTreeNode)

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTableTreeNode
////////////////////////////////////////////////////////////////////////////////////////////////////

PRAGMA_DISABLE_DEPRECATION_WARNINGS

FTableTreeNode::FTableTreeNode(const FName InName, TWeakPtr<FTable> InParentTable, int32 InRowIndex)
	: FBaseTreeNode(InName, false)
	, ParentTable(InParentTable)
	, RowId(InRowIndex)
{
}

FTableTreeNode::FTableTreeNode(const FName InGroupName, TWeakPtr<FTable> InParentTable)
	: FBaseTreeNode(InGroupName, true)
	, ParentTable(InParentTable)
	, RowId(FTableRowId::InvalidRowIndex)
{
}

FTableTreeNode::FTableTreeNode(const FName InName, TWeakPtr<FTable> InParentTable, int32 InRowIndex, bool IsGroup)
	: FBaseTreeNode(InName, IsGroup)
	, ParentTable(InParentTable)
	, RowId(InRowIndex)
{
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

const FText FTableTreeNode::GetDisplayName() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (GetName() != NAME_None)
	{
		return FText::FromName(GetName());
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	return RowId.HasValidIndex() ?
		FText::FromString(FString::Printf(TEXT("row_%d"), RowId.RowIndex)) :
		LOCTEXT("TableTreeNode_InvalidRow", "invalid_row");
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FCustomTableTreeNode
////////////////////////////////////////////////////////////////////////////////////////////////////

PRAGMA_DISABLE_DEPRECATION_WARNINGS

FCustomTableTreeNode::FCustomTableTreeNode(const FName InName, TWeakPtr<FTable> InParentTable, int32 InRowIndex, const FSlateBrush* InIconBrush, FLinearColor InColor, bool IsGroup)
	: FTableTreeNode(InName, InParentTable, InRowIndex, IsGroup)
	, DisplayName(FText::FromName(InName))
	, IconBrush(InIconBrush)
	, IconColor(InColor)
	, Color(InColor)
{
}

FCustomTableTreeNode::FCustomTableTreeNode(const FName InName, TWeakPtr<FTable> InParentTable, int32 InRowIndex, const FSlateBrush* InIconBrush, FLinearColor InIconColor, FLinearColor InColor, bool IsGroup)
	: FTableTreeNode(InName, InParentTable, InRowIndex, IsGroup)
	, DisplayName(FText::FromName(InName))
	, IconBrush(InIconBrush)
	, IconColor(InIconColor)
	, Color(InColor)
{
}

FCustomTableTreeNode::FCustomTableTreeNode(const FName InName, TWeakPtr<FTable> InParentTable, FLinearColor InColor)
	: FTableTreeNode(InName, InParentTable)
	, DisplayName(FText::FromName(InName))
	, IconBrush(FBaseTreeNode::GetDefaultIcon(true))
	, IconColor(InColor)
	, Color(InColor)
{
}

FCustomTableTreeNode::FCustomTableTreeNode(const FName InName, TWeakPtr<FTable> InParentTable, const FSlateBrush* InIconBrush, FLinearColor InColor)
	: FTableTreeNode(InName, InParentTable)
	, DisplayName(FText::FromName(InName))
	, IconBrush(InIconBrush)
	, IconColor(InColor)
	, Color(InColor)
{
}

FCustomTableTreeNode::FCustomTableTreeNode(const FName InName, TWeakPtr<FTable> InParentTable, const FSlateBrush* InIconBrush, FLinearColor InIconColor, FLinearColor InColor)
	: FTableTreeNode(InName, InParentTable)
	, DisplayName(FText::FromName(InName))
	, IconBrush(InIconBrush)
	, IconColor(InIconColor)
	, Color(InColor)
{
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
