// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Table/ViewModels/TreeNodeGrouping.h"

#include "InsightsCore/Common/AsyncOperationProgress.h"
#include "InsightsCore/Table/ViewModels/TableTreeNode.h"

#define LOCTEXT_NAMESPACE "UE::Insights::FTreeNodeGrouping"

namespace UE::Insights
{

INSIGHTS_IMPLEMENT_RTTI(ITreeNodeGrouping)
INSIGHTS_IMPLEMENT_RTTI(FTreeNodeGrouping)
INSIGHTS_IMPLEMENT_RTTI(FTreeNodeGroupingFlat)
INSIGHTS_IMPLEMENT_RTTI(FTreeNodeGroupingByUniqueValue)
INSIGHTS_IMPLEMENT_RTTI(FTreeNodeGroupingByNameFirstLetter)
INSIGHTS_IMPLEMENT_RTTI(FTreeNodeGroupingByType)
INSIGHTS_IMPLEMENT_RTTI(FTreeNodeGroupingByPathBreakdown)

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTreeNodeGrouping
////////////////////////////////////////////////////////////////////////////////////////////////////

FTreeNodeGrouping::FTreeNodeGrouping()
	: Color(FBaseTreeNode::GetDefaultColor(true))
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTreeNodeGrouping::FTreeNodeGrouping(const FText& InShortName, const FText& InTitleName, const FText& InDescription, const FSlateBrush* InIcon)
	: ShortName(InShortName)
	, TitleName(InTitleName)
	, Description(InDescription)
	, Icon(InIcon)
	, Color(FBaseTreeNode::GetDefaultColor(true))
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTreeNodeGrouping::GroupNodes(const TArray<FTableTreeNodePtr>& Nodes, FTableTreeNode& ParentGroup, TWeakPtr<FTable> InParentTable, IAsyncOperationProgress& InAsyncOperationProgress) const
{
	TMap<FName, FTableTreeNodePtr> GroupMap;

	ParentGroup.ClearChildren();

	for (FTableTreeNodePtr NodePtr : Nodes)
	{
		if (InAsyncOperationProgress.ShouldCancelAsyncOp())
		{
			break;
		}

		if (NodePtr->IsGroup())
		{
			ParentGroup.AddChildAndSetParent(NodePtr);
			continue;
		}

		FTableTreeNodePtr GroupPtr = nullptr;

		FTreeNodeGroupInfo GroupInfo = GetGroupForNode(NodePtr);
		FTableTreeNodePtr* GroupPtrPtr = GroupMap.Find(GroupInfo.GetName());
		if (!GroupPtrPtr)
		{
			if (GroupInfo.IsCustom())
			{
				const FSlateBrush* GroupIcon = GroupInfo.GetIcon() != nullptr ? GroupInfo.GetIcon() : FBaseTreeNode::GetDefaultIcon(true);
				GroupPtr = MakeShared<FCustomTableTreeNode>(InParentTable, GroupInfo.GetDisplayName(), GroupIcon, GroupInfo.GetIconColor(), GroupInfo.GetColor());
			}
			else
			{
				GroupPtr = MakeShared<FCustomTableTreeNode>(InParentTable, GroupInfo.GetDisplayName());
			}
			GroupPtr->SetExpansion(GroupInfo.IsExpanded());
			ParentGroup.AddChildAndSetParent(GroupPtr);
			GroupMap.Add(GroupInfo.GetName(), GroupPtr);
		}
		else
		{
			GroupPtr = *GroupPtrPtr;
		}

		GroupPtr->AddChildAndSetParent(NodePtr);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor FTreeNodeGrouping::MakeDistinctColor(const FName& InName)
{
	uint32 Seed = GetTypeHash(InName.ToString());
	return FLinearColor(static_cast<float>(Seed % 3600) / 10.0f, 0.7f, 1.0f, 1.0f).HSVToLinearRGB();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTreeNodeGroupingFlat
////////////////////////////////////////////////////////////////////////////////////////////////////

FTreeNodeGroupingFlat::FTreeNodeGroupingFlat()
	: FTreeNodeGrouping(
		LOCTEXT("Grouping_Flat_ShortName", "All"),
		LOCTEXT("Grouping_Flat_TitleName", "Flat (All)"),
		LOCTEXT("Grouping_Flat_Desc", "Creates a single group. Includes all items."),
		nullptr)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTreeNodeGroupingFlat::GroupNodes(const TArray<FTableTreeNodePtr>& Nodes, FTableTreeNode& ParentGroup, TWeakPtr<FTable> InParentTable, IAsyncOperationProgress& InAsyncOperationProgress) const
{
	ParentGroup.ClearChildren(1);

	auto GroupNode = MakeShared<FCustomTableTreeNode>(InParentTable, LOCTEXT("GroupName_All", "All"));
	GroupNode->SetIconColor(GetColor());
	GroupNode->SetColor(GetColor());
	GroupNode->SetExpansion(true);
	ParentGroup.AddChildAndSetParent(GroupNode);

	GroupNode->ClearChildren(Nodes.Num());
	for (FTableTreeNodePtr ChildNode : Nodes)
	{
		if (InAsyncOperationProgress.ShouldCancelAsyncOp())
		{
			break;
		}

		GroupNode->AddChildAndSetParent(ChildNode);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTreeNodeGroupingByUniqueValue
////////////////////////////////////////////////////////////////////////////////////////////////////

FTreeNodeGroupingByUniqueValue::FTreeNodeGroupingByUniqueValue(TSharedRef<FTableColumn> InColumnRef)
	: FTreeNodeGrouping(
		FText::Format(LOCTEXT("Grouping_ByUniqueValue_ShortNameFmt", "{0} (Unique)"), InColumnRef->GetTitleName()),
		FText::Format(LOCTEXT("Grouping_ByUniqueValue_TitleNameFmt", "Unique Values - {0}"), InColumnRef->GetTitleName()),
		LOCTEXT("Grouping_ByUniqueValue_Desc", "Creates a group for each unique value."),
		nullptr)
	, ColumnRef(InColumnRef)
{
	SetColor(FTreeNodeGrouping::MakeDistinctColor(InColumnRef->GetId()));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTreeNodeGroupInfo FTreeNodeGroupingByUniqueValue::GetGroupForNode(const FBaseTreeNodePtr InNode) const
{
	FTableTreeNodePtr TableTreeNodePtr = StaticCastSharedPtr<FTableTreeNode>(InNode);
	FText ValueAsText = ColumnRef->GetValueAsText(*TableTreeNodePtr);

	FStringView GroupName(ValueAsText.ToString());
	if (GroupName.Len() >= NAME_SIZE)
	{
		GroupName = FStringView(GroupName.GetData(), NAME_SIZE - 1);
	}

	return FTreeNodeGroupInfo(FName(GroupName, 0), false, GetColor());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FCustomTableTreeNode> FTreeNodeGroupingByUniqueValue::CreateGroupNode(TWeakPtr<FTable> InParentTable, const FText& InDisplayName) const
{
	auto GroupNode = MakeShared<FCustomTableTreeNode>(InParentTable, InDisplayName);
	if (GetIcon() != nullptr)
	{
		GroupNode->SetIcon(GetIcon());
	}
	GroupNode->SetIconColor(GetColor());
	GroupNode->SetColor(GetColor());
	GroupNode->SetExpansion(false);
	return GroupNode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FCustomTableTreeNode> FTreeNodeGroupingByUniqueValue::CreateUnsetGroupNode(TWeakPtr<FTable> InParentTable) const
{
	auto GroupNode = MakeShared<FCustomTableTreeNode>(InParentTable, LOCTEXT("GroupName_Unset", "<unset>"));
	GroupNode->SetExpansion(false);
	return GroupNode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// TTreeNodeGroupingByUniqueValue specializations
////////////////////////////////////////////////////////////////////////////////////////////////////

template<> bool TTreeNodeGroupingByUniqueValue<bool>::GetValue(const FTableCellValue& CellValue) { return CellValue.Bool; }
template<> int64 TTreeNodeGroupingByUniqueValue<int64>::GetValue(const FTableCellValue& CellValue) { return CellValue.Int64; }
template<> float TTreeNodeGroupingByUniqueValue<float>::GetValue(const FTableCellValue& CellValue) { return CellValue.Float; }
template<> double TTreeNodeGroupingByUniqueValue<double>::GetValue(const FTableCellValue& CellValue) { return CellValue.Double; }

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTreeNodeGroupingByUniqueValueCString
////////////////////////////////////////////////////////////////////////////////////////////////////

FName FTreeNodeGroupingByUniqueValueCString::GetGroupName(const TCHAR* Value)
{
	FStringView StringView(Value);
	if (StringView.Len() >= NAME_SIZE)
	{
		StringView = FStringView(StringView.GetData(), NAME_SIZE - 1);
	}
	return FName(StringView, 0);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTreeNodeGroupingByUniqueValueCString::GroupNodes(const TArray<FTableTreeNodePtr>& Nodes, FTableTreeNode& ParentGroup, TWeakPtr<FTable> InParentTable, IAsyncOperationProgress& InAsyncOperationProgress) const
{
	FTableTreeNodePtr UnsetGroupPtr = nullptr; // for unset FTableCellValue
	FTableTreeNodePtr EmptyGroupPtr = nullptr; // for nullptr and empty strings
	TMap<const TCHAR*, FTableTreeNodePtr, FDefaultSetAllocator, TStringPointerMapKeyFuncs_DEPRECATED<const TCHAR*, FTableTreeNodePtr>> GroupMap; // for valid strings
	TMap<FName, FTableTreeNodePtr> GroupNameMap; // for valid strings

	ParentGroup.ClearChildren();

	for (FTableTreeNodePtr NodePtr : Nodes)
	{
		if (InAsyncOperationProgress.ShouldCancelAsyncOp())
		{
			break;
		}

		if (NodePtr->IsGroup())
		{
			ParentGroup.AddChildAndSetParent(NodePtr);
			continue;
		}

		FTableTreeNodePtr GroupPtr = nullptr;

		const FTableColumn& Column = *GetColumn();
		const TOptional<FTableCellValue> CellValue = Column.GetValue(*NodePtr);
		if (CellValue.IsSet())
		{
			const TCHAR* Value = CellValue.GetValue().CString;
			if (Value != nullptr && Value[0] != TEXT('\0'))
			{
				FTableTreeNodePtr* GroupPtrPtr = GroupMap.Find(Value);
				if (!GroupPtrPtr)
				{
					const FName GroupName = GetGroupName(Value);
					GroupPtrPtr = GroupNameMap.Find(GroupName);
					if (!GroupPtrPtr)
					{
						auto GroupNode = MakeShared<FCustomTableTreeNode>(InParentTable, FText::FromString(Value));
						GroupNode->SetIconColor(GetColor());
						GroupNode->SetColor(GetColor());
						GroupNode->SetExpansion(false);

						GroupPtr = GroupNode;
						ParentGroup.AddChildAndSetParent(GroupPtr);
						GroupNameMap.Add(GroupName, GroupPtr);
					}
					else
					{
						GroupPtr = *GroupPtrPtr;
					}
					GroupMap.Add(Value, GroupPtr);
				}
				else
				{
					GroupPtr = *GroupPtrPtr;
				}
			}
			else
			{
				if (!EmptyGroupPtr)
				{
					EmptyGroupPtr = MakeShared<FCustomTableTreeNode>(InParentTable, LOCTEXT("GroupName_NA", "N/A"));
					EmptyGroupPtr->SetExpansion(false);
					ParentGroup.AddChildAndSetParent(EmptyGroupPtr);
				}
				GroupPtr = EmptyGroupPtr;
			}
		}
		else
		{
			if (!UnsetGroupPtr)
			{
				UnsetGroupPtr = MakeShared<FCustomTableTreeNode>(InParentTable, LOCTEXT("GroupName_Unset", "<unset>"));
				UnsetGroupPtr->SetExpansion(false);
				ParentGroup.AddChildAndSetParent(UnsetGroupPtr);
			}
			GroupPtr = UnsetGroupPtr;
		}

		GroupPtr->AddChildAndSetParent(NodePtr);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTreeNodeGroupingByNameFirstLetter
////////////////////////////////////////////////////////////////////////////////////////////////////

FTreeNodeGroupingByNameFirstLetter::FTreeNodeGroupingByNameFirstLetter()
	: FTreeNodeGrouping(
		LOCTEXT("Grouping_ByName_ShortName", "Name"),
		LOCTEXT("Grouping_ByName_TitleName", "Name (First Letter)"),
		LOCTEXT("Grouping_ByName_Desc", "Creates a group for each first letter of node names."),
		nullptr)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTreeNodeGroupInfo FTreeNodeGroupingByNameFirstLetter::GetGroupForNode(const FBaseTreeNodePtr InNode) const
{
	return { *InNode->GetDisplayName().ToString().Left(1), false };
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTreeNodeGroupingByType
////////////////////////////////////////////////////////////////////////////////////////////////////

FTreeNodeGroupingByType::FTreeNodeGroupingByType()
	: FTreeNodeGrouping(
		LOCTEXT("Grouping_ByTypeName_ShortName", "TypeName"),
		LOCTEXT("Grouping_ByTypeName_TitleName", "Type Name"),
		LOCTEXT("Grouping_ByTypeName_Desc", "Creates a group for each node type."),
		nullptr)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTreeNodeGroupInfo FTreeNodeGroupingByType::GetGroupForNode(const FBaseTreeNodePtr InNode) const
{
	return { InNode->GetTypeName(), true };
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTreeNodeGroupingByPathBreakdown
////////////////////////////////////////////////////////////////////////////////////////////////////

FTreeNodeGroupingByPathBreakdown::FTreeNodeGroupingByPathBreakdown(TSharedRef<FTableColumn> InColumnRef)
	: FTreeNodeGrouping(
		FText::Format(LOCTEXT("Grouping_ByPathBreakdown_ShortNameFmt", "{0} (Path)"), InColumnRef->GetTitleName()),
		FText::Format(LOCTEXT("Grouping_ByPathBreakdown_TitleNameFmt", "Path Breakdown - {0}"), InColumnRef->GetTitleName()),
		LOCTEXT("Grouping_ByPathBreakdown_Desc", "Creates a tree hierarchy out of the path structure of string values."),
		nullptr)
	, ColumnRef(InColumnRef)
{
	SetColor(FTreeNodeGrouping::MakeDistinctColor(InColumnRef->GetId()));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTreeNodeGroupingByPathBreakdown::GroupNodes(
	const TArray<FTableTreeNodePtr>& Nodes,
	FTableTreeNode& ParentGroup,
	TWeakPtr<FTable> InParentTable,
	IAsyncOperationProgress& InAsyncOperationProgress) const
{
	ParentGroup.ClearChildren();

	struct FDirectory
	{
		FDirectory() = delete;
		FDirectory(FName InName, FDirectory* InParent, FTableTreeNode* InNode)
			: Name(InName)
			, Parent(InParent)
			, Node(InNode)
		{
		}

		FName Name;
		FDirectory* Parent;
		FTableTreeNode* Node;
		TMap<FName, FDirectory*> ChildrenMap; // Directory Name -> FDirectory*
	};

	TArray<FDirectory*> Directories;

	static FName RootName(TEXT("~"));
	FDirectory* Root = new FDirectory(RootName, nullptr, &ParentGroup);
	Directories.Add(Root);

	FTableTreeNode* UnsetGroupPtr = nullptr;
	TMap<FString, FDirectory*> FullNameGroupMap; // FullName -> FDirectory*

	for (FTableTreeNodePtr NodePtr : Nodes)
	{
		if (InAsyncOperationProgress.ShouldCancelAsyncOp())
		{
			break;
		}

		if (NodePtr->IsGroup())
		{
			ParentGroup.AddChildAndSetParent(NodePtr);
			continue;
		}

		FString FullName;
		const TOptional<FTableCellValue> OptionalValue = ColumnRef->GetValue(*NodePtr);
		if (OptionalValue.IsSet())
		{
			FullName = OptionalValue.GetValue().AsString();
		}

		FDirectory* Directory = Root;

		if (!FullName.IsEmpty())
		{
			FDirectory** FoundDir = FullNameGroupMap.Find(FullName);
			if (FoundDir)
			{
				Directory = *FoundDir;
			}
			else
			{
				const TCHAR* Start = *FullName;
				const TCHAR* End = Start;
				while (true)
				{
					if (*End == TEXT('/') || *End == TEXT('\\') || *End == TEXT('\0'))
					{
						const FName DirName(int32(End - Start), Start, 0);

						FoundDir = Directory->ChildrenMap.Find(DirName);
						if (FoundDir)
						{
							Directory = *FoundDir;
						}
						else if (!DirName.IsNone())
						{
							const FText GroupName = FText::FromName(DirName);
							auto GroupNode = MakeShared<FCustomTableTreeNode>(InParentTable, GroupName);
							if (GetIcon() != nullptr)
							{
								GroupNode->SetIcon(GetIcon());
							}
							GroupNode->SetIconColor(GetColor());
							GroupNode->SetColor(GetColor());
							GroupNode->SetExpansion(false);
							Directory->Node->AddChildAndSetParent(GroupNode);
							Directory = new FDirectory(DirName, Directory, &GroupNode.Get());
							Directory->Parent->ChildrenMap.Add(DirName, Directory);
							Directories.Add(Directory);
						}

						if (*End == TEXT('\0'))
						{
							FullNameGroupMap.Add(FullName, Directory);
							break;
						}

						Start = End + 1;
					}
					++End;
				}
			}
		}

		if (Directory != Root)
		{
			Directory->Node->AddChildAndSetParent(NodePtr);
		}
		else
		{
			if (!UnsetGroupPtr)
			{
				auto GroupNode = MakeShared<FCustomTableTreeNode>(InParentTable, LOCTEXT("GroupName_NA", "N/A"));
				GroupNode->SetExpansion(false);
				ParentGroup.AddChildAndSetParent(GroupNode);
				UnsetGroupPtr = &GroupNode.Get();
			}
			UnsetGroupPtr->AddChildAndSetParent(NodePtr);
		}
	}

	// Clean up.
	for (FDirectory* Dir : Directories)
	{
		delete Dir;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
