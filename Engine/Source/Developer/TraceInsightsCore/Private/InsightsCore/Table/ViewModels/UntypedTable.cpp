// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Table/ViewModels/UntypedTable.h"

// TraceServices
#include "TraceServices/Containers/Tables.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TableCellValueFormatter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueGetter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"
#include "InsightsCore/Table/ViewModels/TableTreeNode.h"

#define LOCTEXT_NAMESPACE "UE::Insights::FUntypedTable"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FUntypedTableTreeNodeValueGetter
////////////////////////////////////////////////////////////////////////////////////////////////////

class FUntypedTableTreeNodeValueGetter : public FTableCellValueGetter
{
public:
	FUntypedTableTreeNodeValueGetter(ETableCellDataType InDataType) : FTableCellValueGetter(), DataType(InDataType) {}

	virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
	{
		ensure(Node.Is<FTableTreeNode>());
		const FTableTreeNode& TableTreeNode = static_cast<const FTableTreeNode&>(Node);

		if (!Node.IsGroup()) // Table Row Node
		{
			const TSharedPtr<FTable> TablePtr = Column.GetParentTable().Pin();
			const TSharedPtr<FUntypedTable> UntypedTablePtr = StaticCastSharedPtr<FUntypedTable>(TablePtr);
			if (UntypedTablePtr.IsValid())
			{
				TSharedPtr<TraceServices::IUntypedTableReader> Reader = UntypedTablePtr->GetTableReader();
				if (Reader.IsValid() && TableTreeNode.GetRowId().HasValidIndex())
				{
					Reader->SetRowIndex(TableTreeNode.GetRowId().RowIndex);
					const int32 ColumnIndex = Column.GetIndex();
					switch (DataType)
					{
						case ETableCellDataType::Bool:    return TOptional<FTableCellValue>(FTableCellValue(Reader->GetValueBool(ColumnIndex)));
						case ETableCellDataType::Int64:   return TOptional<FTableCellValue>(FTableCellValue(Reader->GetValueInt(ColumnIndex)));
						case ETableCellDataType::Float:   return TOptional<FTableCellValue>(FTableCellValue(Reader->GetValueFloat(ColumnIndex)));
						case ETableCellDataType::Double:  return TOptional<FTableCellValue>(FTableCellValue(Reader->GetValueDouble(ColumnIndex)));
						case ETableCellDataType::CString: return TOptional<FTableCellValue>(FTableCellValue(Reader->GetValueCString(ColumnIndex)));
					}
				}
			}
		}
		else // Aggregated Group Node
		{
			if (Column.GetAggregation() != ETableColumnAggregation::None)
			{
				const FTableCellValue* ValuePtr = TableTreeNode.FindAggregatedValue(Column.GetId());
				if (ValuePtr != nullptr)
				{
					ensure(ValuePtr->DataType == DataType);
					return TOptional<FTableCellValue>(*ValuePtr);
				}
			}
		}

		return TOptional<FTableCellValue>();
	}

private:
	ETableCellDataType DataType;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// FUntypedTable
////////////////////////////////////////////////////////////////////////////////////////////////////

FUntypedTable::FUntypedTable()
	: SourceTable()
	, TableReader()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FUntypedTable::~FUntypedTable()
{
	Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUntypedTable::Reset()
{
	SourceTable.Reset();
	TableReader.Reset();

	FTable::Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool AreTableLayoutsEqual(const TraceServices::ITableLayout& TableLayoutA, const TraceServices::ITableLayout& TableLayoutB)
{
	if (TableLayoutA.GetColumnCount() != TableLayoutB.GetColumnCount())
	{
		return false;
	}

	int32 ColumnCount = static_cast<int32>(TableLayoutA.GetColumnCount());
	for (int32 ColumnIndex = 0; ColumnIndex < ColumnCount; ++ColumnIndex)
	{
		if (TableLayoutA.GetColumnType(ColumnIndex) != TableLayoutB.GetColumnType(ColumnIndex))
		{
			return false;
		}
		if (FCString::Strcmp(TableLayoutA.GetColumnName(ColumnIndex), TableLayoutB.GetColumnName(ColumnIndex)) != 0)
		{
			return false;
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FUntypedTable::UpdateSourceTable(TSharedPtr<TraceServices::IUntypedTable> InSourceTable)
{
	bool bTableLayoutChanged;

	if (InSourceTable.IsValid())
	{
		bTableLayoutChanged = !SourceTable.IsValid() || !AreTableLayoutsEqual(InSourceTable->GetLayout(), SourceTable->GetLayout());
		SourceTable = InSourceTable;
		TableReader = MakeShareable(SourceTable->CreateReader());
	}
	else
	{
		bTableLayoutChanged = SourceTable.IsValid();
		SourceTable.Reset();
		TableReader.Reset();
	}

	if (bTableLayoutChanged)
	{
		ResetColumns();
		if (SourceTable.IsValid())
		{
			CreateColumns(SourceTable->GetLayout());
		}
	}

	return bTableLayoutChanged;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUntypedTable::CreateColumns(const TraceServices::ITableLayout& TableLayout)
{
	ensure(GetColumnCount() == 0);
	const int32 ColumnCount = static_cast<int32>(TableLayout.GetColumnCount());

	//////////////////////////////////////////////////
	// Hierarchy Column

	int32 HierarchyColumnIndex = -1;
	const TCHAR* HierarchyColumnName = nullptr;

	// Look for first string column.
	//for (int32 ColumnIndex = 0; ColumnIndex < ColumnCount; ++ColumnIndex)
	//{
	//	TraceServices::ETableColumnType ColumnType = TableLayout.GetColumnType(ColumnIndex);
	//	if (ColumnType == TraceServices::TableColumnType_CString)
	//	{
	//		HierarchyColumnIndex = ColumnIndex;
	//		HierarchyColumnName = TableLayout.GetColumnName(ColumnIndex);
	//		break;
	//	}
	//}

	AddHierarchyColumn(HierarchyColumnIndex, HierarchyColumnName);

	//////////////////////////////////////////////////

	for (int32 ColumnIndex = 0; ColumnIndex < ColumnCount; ++ColumnIndex)
	{
		TraceServices::ETableColumnType ColumnType = TableLayout.GetColumnType(ColumnIndex);
		uint32 ColumnDisplayHintFlags = TableLayout.GetColumnDisplayHintFlags(ColumnIndex);
		const TCHAR* ColumnName = TableLayout.GetColumnName(ColumnIndex);

		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FName(ColumnName));
		FTableColumn& Column = *ColumnRef;

		const FString ColumnNameStr(ColumnName);
		const FText ColumnNameText = FText::FromString(ColumnNameStr);

		ETableColumnFlags ColumnFlags = ETableColumnFlags::CanBeFiltered | ETableColumnFlags::CanBeHidden;
		if (ColumnIndex != HierarchyColumnIndex)
		{
			ColumnFlags |= ETableColumnFlags::ShouldBeVisible;
		}

		EHorizontalAlignment HorizontalAlignment = HAlign_Left;
		float InitialColumnWidth = 60.0f;

		ETableColumnAggregation Aggregation = ETableColumnAggregation::None;

		TSharedPtr<ITableCellValueFormatter> FormatterPtr;
		TSharedPtr<ITableCellValueSorter> SorterPtr;
		EColumnSortMode::Type InitialSortMode = EColumnSortMode::Ascending;

		switch (ColumnType)
		{
		case TraceServices::TableColumnType_Bool:
			Column.SetDataType(ETableCellDataType::Bool);
			HorizontalAlignment = HAlign_Right;
			InitialColumnWidth = 40.0f;
			//TODO: if (Hint == AsOnOff)
			//else // if (Hint == AsTrueFalse)
			FormatterPtr = MakeShared<FBoolValueFormatterAsTrueFalse>();
			SorterPtr = MakeShared<FSorterByBoolValue>(ColumnRef);
			InitialSortMode = EColumnSortMode::Ascending;
			break;

		case TraceServices::TableColumnType_Int:
			Column.SetDataType(ETableCellDataType::Int64);
			if (ColumnDisplayHintFlags & TraceServices::TableColumnDisplayHint_Summable)
			{
				Aggregation = ETableColumnAggregation::Sum;
			}
			HorizontalAlignment = HAlign_Right;
			InitialColumnWidth = 60.0f;
			if (ColumnDisplayHintFlags & TraceServices::TableColumnDisplayHint_Memory)
			{
				FormatterPtr = MakeShared<FInt64ValueFormatterAsMemory>();
			}
			else
			{
				FormatterPtr = MakeShared<FInt64ValueFormatterAsNumber>();
			}
			SorterPtr = MakeShared<FSorterByInt64Value>(ColumnRef);
			InitialSortMode = EColumnSortMode::Descending;
			break;

		case TraceServices::TableColumnType_Float:
			Column.SetDataType(ETableCellDataType::Float);
			if (ColumnDisplayHintFlags & TraceServices::TableColumnDisplayHint_Summable)
			{
				Aggregation = ETableColumnAggregation::Sum;
			}
			HorizontalAlignment = HAlign_Right;
			InitialColumnWidth = 60.0f;
			if (ColumnDisplayHintFlags & TraceServices::TableColumnDisplayHint_Time)
			{
				FormatterPtr = MakeShared<FFloatValueFormatterAsTimeAuto>();
				InitialSortMode = EColumnSortMode::Ascending;
			}
			else
			{
				FormatterPtr = MakeShared<FFloatValueFormatterAsNumber>();
				InitialSortMode = EColumnSortMode::Descending;
			}
			SorterPtr = MakeShared<FSorterByFloatValue>(ColumnRef);
			break;

		case TraceServices::TableColumnType_Double:
			Column.SetDataType(ETableCellDataType::Double);
			if (ColumnDisplayHintFlags & TraceServices::TableColumnDisplayHint_Summable)
			{
				Aggregation = ETableColumnAggregation::Sum;
			}
			HorizontalAlignment = HAlign_Right;
			InitialColumnWidth = 80.0f;
			if (ColumnDisplayHintFlags & TraceServices::TableColumnDisplayHint_Time)
			{
				FormatterPtr = MakeShared<FDoubleValueFormatterAsTimeAuto>();
				InitialSortMode = EColumnSortMode::Ascending;
			}
			else
			{
				FormatterPtr = MakeShared<FDoubleValueFormatterAsNumber>();
				InitialSortMode = EColumnSortMode::Descending;
			}
			SorterPtr = MakeShared<FSorterByDoubleValue>(ColumnRef);
			break;

		case TraceServices::TableColumnType_CString:
			Column.SetDataType(ETableCellDataType::CString);
			HorizontalAlignment = HAlign_Left;
			InitialColumnWidth = FMath::Max(120.0f, 6.0f * static_cast<float>(ColumnNameStr.Len()));
			FormatterPtr = MakeShared<FCStringValueFormatterAsText>();
			SorterPtr = MakeShared<FSorterByCStringValue>(ColumnRef);
			InitialSortMode = EColumnSortMode::Ascending;
			break;
		}

		Column.SetIndex(ColumnIndex);

		Column.SetShortName(ColumnNameText);
		Column.SetTitleName(ColumnNameText);

		//TODO: Column.SetDescription(...);

		Column.SetFlags(ColumnFlags);

		Column.SetHorizontalAlignment(HorizontalAlignment);
		Column.SetInitialWidth(InitialColumnWidth);

		Column.SetAggregation(Aggregation);

		Column.SetValueGetter(MakeShared<FUntypedTableTreeNodeValueGetter>(Column.GetDataType()));

		if (FormatterPtr.IsValid())
		{
			Column.SetValueFormatter(FormatterPtr.ToSharedRef());
		}

		Column.SetValueSorter(SorterPtr);
		Column.SetInitialSortMode(InitialSortMode);

		AddColumn(ColumnRef);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
