// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Table/Widgets/SAsyncOperationStatus.h"

#include "Styling/AppStyle.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#include "InsightsCore/Common/IAsyncOperationStatusProvider.h"
#include "InsightsCore/Common/TimeUtils.h"

#define LOCTEXT_NAMESPACE "UE::Insights::SAsyncOperationStatus"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

void SAsyncOperationStatus::Construct(const FArguments& InArgs, TSharedRef<IAsyncOperationStatusProvider> InStatusProvider)
{
	StatusProvider = InStatusProvider;

	ChildSlot
	[
		SNew(SBox)
		.Visibility(this, &SAsyncOperationStatus::GetContentVisibility)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("PopupText.Background"))
			.BorderBackgroundColor(this, &SAsyncOperationStatus::GetBackgroundColorAndOpacity)
			.ToolTipText(this, &SAsyncOperationStatus::GetTooltipText)
			.Padding(FMargin(16.0f, 8.0f, 16.0f, 8.0f))
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
					.Text(this, &SAsyncOperationStatus::GetText)
					.ColorAndOpacity(this, &SAsyncOperationStatus::GetTextColorAndOpacity)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.MinDesiredWidth(16.0f)
					.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
					.Text(this, &SAsyncOperationStatus::GetAnimatedText)
					.ColorAndOpacity(this, &SAsyncOperationStatus::GetTextColorAndOpacity)
				]
			]
		]
	];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EVisibility SAsyncOperationStatus::GetContentVisibility() const
{
	TSharedPtr<IAsyncOperationStatusProvider> StatusProviderSharedPtr = StatusProvider.Pin();
	if (StatusProviderSharedPtr.IsValid())
	{
		return StatusProviderSharedPtr->IsRunning() ? EVisibility::Visible : EVisibility::Collapsed;
	}
	
	return EVisibility::Collapsed;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor SAsyncOperationStatus::GetBackgroundColorAndOpacity() const
{
	const float Opacity = ComputeOpacity();
	return FSlateColor(FLinearColor(0.2f, 0.2f, 0.2f, Opacity));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor SAsyncOperationStatus::GetTextColorAndOpacity() const
{
	const float Opacity = ComputeOpacity();
	return FSlateColor(FLinearColor(1.0f, 1.0f, 1.0f, Opacity));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

float SAsyncOperationStatus::ComputeOpacity() const
{
	TSharedPtr<IAsyncOperationStatusProvider> StatusProviderSharedPtr = StatusProvider.Pin();
	const double TotalDuration = StatusProviderSharedPtr.IsValid() ? StatusProviderSharedPtr->GetAllOperationsDuration() : 0;
	constexpr double FadeInStartTime = 0.1; // [second]
	constexpr double FadeInEndTime = 3.0; // [second]
	constexpr double FadeInDuration = FadeInEndTime - FadeInStartTime;
	const float Opacity = static_cast<float>(FMath::Clamp(TotalDuration - FadeInStartTime, 0.0, FadeInDuration) / FadeInDuration);
	return Opacity;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SAsyncOperationStatus::GetText() const
{
	TSharedPtr<IAsyncOperationStatusProvider> StatusProviderSharedPtr = StatusProvider.Pin();
	FText OperationName = StatusProviderSharedPtr->GetCurrentOperationName();

	if (OperationName.IsEmpty())
	{
		return LOCTEXT("DefaultText", "Computing");
	}
	else
	{
		return OperationName;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SAsyncOperationStatus::GetAnimatedText() const
{
	TSharedPtr<IAsyncOperationStatusProvider> StatusProviderSharedPtr = StatusProvider.Pin();
	const double TotalDuration = StatusProviderSharedPtr.IsValid() ? StatusProviderSharedPtr->GetAllOperationsDuration() : 0;
	const TCHAR* Anim[] = { TEXT(""), TEXT("."), TEXT(".."), TEXT("..."), };
	int32 AnimIndex = static_cast<int32>(TotalDuration / 0.2) % UE_ARRAY_COUNT(Anim);
	return FText::FromString(FString(Anim[AnimIndex]));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SAsyncOperationStatus::GetTooltipText() const
{
	TSharedPtr<IAsyncOperationStatusProvider> StatusProviderSharedPtr = StatusProvider.Pin();
	if (StatusProviderSharedPtr.IsValid())
	{
		const double TotalDuration = StatusProviderSharedPtr->GetAllOperationsDuration();
		const uint32 OperationCount = StatusProviderSharedPtr->GetOperationCount();
		FText CurrentOpName = StatusProviderSharedPtr->GetCurrentOperationName();

		if (OperationCount == 1)
		{
			return FText::Format(LOCTEXT("DefaultTooltip_Fmt2", "{0}...\nElapsed Time: {1}"),
				CurrentOpName,
				FText::FromString(FormatTime(TotalDuration, FTimeValue::Second)));
		}
		else
		{
			const double Duration = StatusProviderSharedPtr->GetCurrentOperationDuration();
			return FText::Format(LOCTEXT("DefaultTooltip_Fmt3", "{0}...\nElapsed Time: {1}\nElapsed Time (op {2}): {3}"),
				CurrentOpName,
				FText::FromString(FormatTime(TotalDuration, FTimeValue::Second)),
				FText::AsNumber(OperationCount),
				FText::FromString(FormatTime(Duration, FTimeValue::Second)));
		}
	}

	return FText::FromString(TEXT(""));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
