// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Table/Widgets/STableTreeView.h"

#include "DesktopPlatformModule.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformFileManager.h"
#include "Logging/MessageLog.h"
#include "SlateOptMacros.h"
#include "Styling/StyleColors.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Views/STableViewBase.h"

#include "InsightsCore/Common/InsightsCoreStyle.h"
#include "InsightsCore/Common/Log.h"
#include "InsightsCore/Filter/ViewModels/FilterConfigurator.h"
#include "InsightsCore/Table/ViewModels/Table.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"
#include "InsightsCore/Table/ViewModels/TreeNodeGrouping.h"
#include "InsightsCore/Table/Widgets/SAsyncOperationStatus.h"
#include "InsightsCore/Table/Widgets/STableTreeViewTooltip.h"
#include "InsightsCore/Table/Widgets/STableTreeViewRow.h"

#include <limits>

#define LOCTEXT_NAMESPACE "UE::Insights::STableTreeView"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTableTreeViewCommands
////////////////////////////////////////////////////////////////////////////////////////////////////

class FTableTreeViewCommands : public TCommands<FTableTreeViewCommands>
{
public:
	FTableTreeViewCommands()
	: TCommands<FTableTreeViewCommands>(
		TEXT("TableTreeViewCommands"),
		NSLOCTEXT("Contexts", "TableTreeViewCommands", "Insights - Table Tree View"),
		NAME_None,
		FInsightsCoreStyle::GetStyleSetName())
	{
	}

	virtual ~FTableTreeViewCommands()
	{
	}

	// UI_COMMAND takes long for the compiler to optimize
	UE_DISABLE_OPTIMIZATION_SHIP
	virtual void RegisterCommands() override
	{
		UI_COMMAND(Command_CopyToClipboard,
			"Copy",
			"Copies the selected table rows to clipboard.",
			EUserInterfaceActionType::Button,
			FInputChord(EModifierKey::Control, EKeys::C));

		UI_COMMAND(Command_CopyColumnToClipboard,
			"Copy Value",
			"Copies the value of the hovered cell to clipboard.",
			EUserInterfaceActionType::Button,
			FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::C));

		UI_COMMAND(Command_CopyColumnTooltipToClipboard,
			"Copy Tooltip",
			"Copies the tooltip of the hovered cell to clipboard.",
			EUserInterfaceActionType::Button,
			FInputChord(EModifierKey::Control | EModifierKey::Alt, EKeys::C));

		UI_COMMAND(Command_ExpandSubtree,
			"Expand Subtree",
			"Expand the subtree that starts from the selected group node.",
			EUserInterfaceActionType::Button,
			FInputChord(EModifierKey::None, EKeys::E));

		UI_COMMAND(Command_ExpandCriticalPath,
			"Expand Critical Path",
			"Expand the first group child node recursively until a leaf nodes in reached.",
			EUserInterfaceActionType::Button,
			FInputChord(EModifierKey::None, EKeys::R));

		UI_COMMAND(Command_CollapseSubtree,
			"Collapse Subtree",
			"Collapse the subtree that starts from the selected group node.",
			EUserInterfaceActionType::Button,
			FInputChord(EModifierKey::None, EKeys::C));

		UI_COMMAND(Command_ExportToFile,
			"Export Visible Tree to File...",
			"Exports the tree/table content to a file. It exports only the tree nodes currently expanded in the tree, including leaf nodes.",
			EUserInterfaceActionType::Button,
			FInputChord(EModifierKey::Control, EKeys::E));

		UI_COMMAND(Command_ExportEntireTreeToFile,
			"Export Entire Tree (+ Leaf Nodes) to File...",
			"Exports the entire tree/table content to a file. It exports also the collapsed tree nodes, including the leaf nodes. Filtered out nodes are not exported.",
			EUserInterfaceActionType::Button,
			FInputChord(EModifierKey::Shift, EKeys::E));

		UI_COMMAND(Command_ExportEntireTreeToFileExceptLeaves,
			"Export Entire Tree (- Leaf Nodes) to File...",
			"Exports the entire tree/table content to a file, but not the leaf nodes. It exports the collapsed tree nodes. Filtered out nodes are not exported.",
			EUserInterfaceActionType::Button,
			FInputChord(EModifierKey::Alt, EKeys::E));
	}
	UE_ENABLE_OPTIMIZATION_SHIP

	TSharedPtr<FUICommandInfo> Command_CopyToClipboard;
	TSharedPtr<FUICommandInfo> Command_CopyColumnToClipboard;
	TSharedPtr<FUICommandInfo> Command_CopyColumnTooltipToClipboard;
	TSharedPtr<FUICommandInfo> Command_ExpandSubtree;
	TSharedPtr<FUICommandInfo> Command_ExpandCriticalPath;
	TSharedPtr<FUICommandInfo> Command_CollapseSubtree;
	TSharedPtr<FUICommandInfo> Command_ExportToFile;
	TSharedPtr<FUICommandInfo> Command_ExportEntireTreeToFile;
	TSharedPtr<FUICommandInfo> Command_ExportEntireTreeToFileExceptLeaves;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// STableTreeView
////////////////////////////////////////////////////////////////////////////////////////////////////

STableTreeView::STableTreeView()
	: Root(MakeShared<FCustomTableTreeNode>(Table, LOCTEXT("GroupName_Root", "Root")))
	, FilteredNodesPtr(&TableRowNodes)
	, ColumnBeingSorted(GetDefaultColumnBeingSorted())
	, ColumnSortMode(GetDefaultColumnSortMode())
{
	check(Root->IsGroup());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

STableTreeView::~STableTreeView()
{
	if (bRunInAsyncMode && !bIsCloseScheduled)
	{
		UE_LOGF(LogInsights, Log, "TableTreeView running in async mode was closed but OnClose() was not called. Call OnClose() from the owner tab/window.");
	}

	// Backup call to OnClose() in case it was not called from the owner.
	OnClose();

	if (CurrentAsyncOpFilterConfigurator)
	{
		delete CurrentAsyncOpFilterConfigurator;
		CurrentAsyncOpFilterConfigurator = nullptr;
	}

	//FTableTreeViewCommands::Unregister();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::SetCurrentGroupings(TArray<TSharedPtr<FTreeNodeGrouping>>& InCurrentGroupings)
{
	PreChangeGroupings();
	CurrentGroupings = InCurrentGroupings;
	PostChangeGroupings();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::InitCommandList()
{
	FTableTreeViewCommands::Register();
	CommandList = MakeShared<FUICommandList>();

	CommandList->MapAction(FTableTreeViewCommands::Get().Command_CopyToClipboard,
		FExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_CopySelectedToClipboard_Execute),
		FCanExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_CopySelectedToClipboard_CanExecute));

	CommandList->MapAction(FTableTreeViewCommands::Get().Command_CopyColumnToClipboard,
		FExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_CopyColumnToClipboard_Execute),
		FCanExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_CopyColumnToClipboard_CanExecute));

	CommandList->MapAction(FTableTreeViewCommands::Get().Command_CopyColumnTooltipToClipboard,
		FExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_CopyColumnTooltipToClipboard_Execute),
		FCanExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_CopyColumnTooltipToClipboard_CanExecute));

	CommandList->MapAction(FTableTreeViewCommands::Get().Command_ExpandSubtree,
		FExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_ExpandSubtree_Execute),
		FCanExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_ExpandSubtree_CanExecute));

	CommandList->MapAction(FTableTreeViewCommands::Get().Command_ExpandCriticalPath,
		FExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_ExpandCriticalPath_Execute),
		FCanExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_ExpandCriticalPath_CanExecute));

	CommandList->MapAction(FTableTreeViewCommands::Get().Command_CollapseSubtree,
		FExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_CollapseSubtree_Execute),
		FCanExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_CollapseSubtree_CanExecute));

	CommandList->MapAction(FTableTreeViewCommands::Get().Command_ExportToFile,
		FExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_ExportToFile_Execute, false, true),
		FCanExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_ExportToFile_CanExecute));

	CommandList->MapAction(FTableTreeViewCommands::Get().Command_ExportEntireTreeToFile,
		FExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_ExportToFile_Execute, true, true),
		FCanExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_ExportToFile_CanExecute));

	CommandList->MapAction(FTableTreeViewCommands::Get().Command_ExportEntireTreeToFileExceptLeaves,
		FExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_ExportToFile_Execute, true, false),
		FCanExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_ExportToFile_CanExecute));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::Construct(const FArguments& InArgs, TSharedPtr<FTable> InTablePtr)
{
	ConstructWidget(InTablePtr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void STableTreeView::ConstructWidget(TSharedPtr<FTable> InTablePtr)
{
	check(InTablePtr.IsValid());
	Table = InTablePtr;

	InitAvailableViewPresets();

	SAssignNew(ExternalScrollbar, SScrollBar)
	.AlwaysShowScrollbar(true);

	auto WidgetContent = SNew(SVerticalBox);

	ConstructHeaderArea(WidgetContent);

	// Tree view
	WidgetContent->AddSlot()
	.FillHeight(1.0f)
	.Padding(0.0f, 6.0f, 0.0f, 0.0f)
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.Padding(0.0f)
		[
			SNew(SOverlay)

			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SAssignNew(TreeView, STreeView<FTableTreeNodePtr>)
				.ExternalScrollbar(ExternalScrollbar)
				.SelectionMode(ESelectionMode::Multi)
				.TreeItemsSource(&FilteredGroupNodes)
				.OnGetChildren(this, &STableTreeView::TreeView_OnGetChildren)
				.OnGenerateRow(this, &STableTreeView::TreeView_OnGenerateRow)
				.OnSelectionChanged(this, &STableTreeView::TreeView_OnSelectionChanged)
				.OnExpansionChanged(this, &STableTreeView::TreeView_OnExpansionChanged)
				.OnMouseButtonDoubleClick(this, &STableTreeView::TreeView_OnMouseButtonDoubleClick)
				.OnContextMenuOpening(FOnContextMenuOpening::CreateSP(this, &STableTreeView::TreeView_GetMenuContent))
				.HeaderRow
				(
					SAssignNew(TreeViewHeaderRow, SHeaderRow)
					.Visibility(EVisibility::Visible)
				)
			]

			+ SOverlay::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Bottom)
			.Padding(16.0f)
			[
				SAssignNew(AsyncOperationStatus, SAsyncOperationStatus, SharedThis(this))
			]

			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			.Padding(FMargin(0.0f, 30.0f, 0.0f, 0.0f))
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
				.Visibility_Lambda([this]() -> EVisibility
					{
						return TreeViewBannerText.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
					})
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(4.0f, 0.0f, 4.0f, 0.0f)
					[
						SNew(SImage)
						.Image(FInsightsCoreStyle::GetBrush("TreeViewBanner.WarningIcon"))
					]

					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(4.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(STextBlock)
						.Text(this, &STableTreeView::GetTreeViewBannerText)
						.ColorAndOpacity(FSlateColor(EStyleColor::Foreground))
					]
				]
			]
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f)
		[
			SNew(SBox)
			.WidthOverride(FOptionalSize(13.0f))
			[
				ExternalScrollbar.ToSharedRef()
			]
		]
	];

	ConstructFooterArea(WidgetContent);

	ChildSlot
	[
		WidgetContent
	];

	InitNodeFiltering();
	InitHierarchyFiltering();
	InitializeAndShowHeaderColumns();
	InitCommandList();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> STableTreeView::ConstructHierarchyBreadcrumbTrail()
{
	TSharedRef<SWidget> Widget = SNew(SOverlay)

		+ SOverlay::Slot()
		[
			SNew(SComboButton)
			.HasDownArrow(false)
			.ToolTipText(LOCTEXT("GroupingsTooltip", "Groupings Hierarchy"))
			.OnGetMenuContent_Lambda([this]() -> TSharedRef<SWidget>
				{
					if (CurrentGroupings.Num() > 0)
					{
						return GetGroupingCrumbMenuContent(CurrentGroupings.Last());
					}
					else
					{
						return SNew(SBox);
					}
				})
			.OnComboBoxOpened_Lambda([this]()
				{
					if (CurrentGroupings.Num() > 0)
					{
						OnGroupingCrumbClicked(CurrentGroupings.Last());
					}
				})
			.ButtonContent()
			[
				SNew(SBox)
				.HAlign(HAlign_Left)
				[
					SAssignNew(GroupingBreadcrumbTrail, SBreadcrumbTrail<TSharedPtr<FTreeNodeGrouping>>)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.TextStyle(FAppStyle::Get(), "NormalText")
					.DelimiterImage(FAppStyle::Get().GetBrush("Icons.ChevronRight"))
					.ButtonContentPadding(FMargin(3.0f, 2.0f))
					.GetCrumbButtonContent(this, &STableTreeView::GetGroupingCrumbButtonContent)
					.GetCrumbMenuContent(this, &STableTreeView::GetGroupingCrumbMenuContent)
					.OnCrumbClicked(this, &STableTreeView::OnGroupingCrumbClicked)
					.ToolTipText(LOCTEXT("GroupingCrumbTooltip", "Click a grouping to remove the additional groupings.\nClick a separator to add/insert a new grouping or to change the current groupings."))
				]
			]
		]

		+ SOverlay::Slot()
		[
			SNew(SBox)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.0f, 0.0f, 6.0f, 0.0f))
			.Visibility_Lambda([this]() ->EVisibility
				{
					return (CurrentGroupings.Num() == 1) ? EVisibility::HitTestInvisible : EVisibility::Collapsed;
				})
			[
				SNew(STextBlock)
				.ColorAndOpacity(FSlateColor(EStyleColor::White25))
				.Text(LOCTEXT("GroupingsHintText", "Click to add or edit groupings"))
			]
		];

	GroupingBreadcrumbTrail->ScrollToEnd();
	return Widget;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedPtr<SWidget> STableTreeView::ConstructFilterToolbar()
{
	TSharedPtr<SHorizontalBox> Box = SNew(SHorizontalBox);

	Box->AddSlot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		[
			ConstructSearchBox()
		];

	Box->AddSlot()
		.AutoWidth()
		.Padding(4.0f, 0.0f, 0.0f, 0.0f)
		.VAlign(VAlign_Center)
		[
			ConstructFilterConfiguratorButton()
		];

	return Box;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void STableTreeView::ConstructHeaderArea(TSharedRef<SVerticalBox> InHostBox)
{
	TSharedPtr<SWidget> FilterToolbarWidget = ConstructFilterToolbar();
	if (FilterToolbarWidget)
	{
		InHostBox->AddSlot()
			.VAlign(VAlign_Center)
			.AutoHeight()
			.Padding(2.0f)
			[
				FilterToolbarWidget.ToSharedRef()
			];
	}

	TSharedPtr<SWidget> HierarchyBreadcrumbTrailWidget = ConstructHierarchyBreadcrumbTrail();
	TSharedPtr<SWidget> ToolbarWidget = ConstructToolbar();
	if (HierarchyBreadcrumbTrailWidget && ToolbarWidget)
	{
		InHostBox->AddSlot()
			.VAlign(VAlign_Center)
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					HierarchyBreadcrumbTrailWidget.ToSharedRef()
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					ToolbarWidget.ToSharedRef()
				]
			];
	}
	else if (HierarchyBreadcrumbTrailWidget)
	{
		InHostBox->AddSlot()
			.VAlign(VAlign_Center)
			.AutoHeight()
			.Padding(2.0f)
			[
				HierarchyBreadcrumbTrailWidget.ToSharedRef()
			];
	}
	else if (ToolbarWidget)
	{
		InHostBox->AddSlot()
			.VAlign(VAlign_Center)
			.AutoHeight()
			.Padding(2.0f)
			[
				ToolbarWidget.ToSharedRef()
			];
	}

	TSharedPtr<SWidget> ExtraToolbarWidget = ConstructExtraToolbar();
	if (ExtraToolbarWidget)
	{
		InHostBox->AddSlot()
			.VAlign(VAlign_Center)
			.AutoHeight()
			.Padding(2.0f)
			[
				ExtraToolbarWidget.ToSharedRef()
			];
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void STableTreeView::ConstructFooterArea(TSharedRef<SVerticalBox> InHostBox)
{
	TSharedPtr<SWidget> FooterWidget = ConstructFooter();
	if (FooterWidget.IsValid())
	{
		InHostBox->AddSlot()
			.AutoHeight()
			.Padding(0.0f, 6.0f, 0.0f, 0.0f)
			[
				FooterWidget.ToSharedRef()
			];
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SWidget> STableTreeView::TreeView_GetMenuContent()
{
	FSlateApplication::Get().CloseToolTip();

	TArray<FTableTreeNodePtr> SelectedNodes;
	const int32 NumSelectedNodes = TreeView->GetSelectedItems(SelectedNodes);
	FTableTreeNodePtr SelectedNode = NumSelectedNodes ? SelectedNodes[0] : nullptr;

	FText SelectionStr;
	if (NumSelectedNodes == 0)
	{
		SelectionStr = LOCTEXT("NothingSelected", "Nothing selected");
	}
	else if (NumSelectedNodes == 1)
	{
		FString ItemName = SelectedNode->GetDisplayName().ToString();
		const int32 MaxStringLen = 64;
		if (ItemName.Len() > MaxStringLen)
		{
			ItemName = ItemName.Left(MaxStringLen) + TEXT("...");
		}
		SelectionStr = FText::FromString(ItemName);
	}
	else
	{
		SelectionStr = FText::Format(LOCTEXT("MultipleSelection_Fmt", "{0} selected items"), FText::AsNumber(NumSelectedNodes));
	}

	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList.ToSharedRef());

	MenuBuilder.SetExtendersEnabled(true);

	TSharedRef<FExtender> Extender = MakeShared<FExtender>();
	MenuBuilder.PushExtender(Extender);

	ExtendMenu(Extender);

	// Selection menu
	MenuBuilder.BeginSection("Selection", LOCTEXT("ContextMenu_Section_Selection", "Selection"));
	{
		struct FLocal
		{
			static bool ReturnFalse()
			{
				return false;
			}
		};

		FUIAction DummyUIAction;
		DummyUIAction.CanExecuteAction = FCanExecuteAction::CreateStatic(&FLocal::ReturnFalse);
		MenuBuilder.AddMenuEntry
		(
			SelectionStr,
			LOCTEXT("ContextMenu_Selection", "Currently selected items"),
			FSlateIcon(),
			DummyUIAction,
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Node", LOCTEXT("ContextMenu_Section_Node", "Node"));
	{
		MenuBuilder.AddMenuEntry
		(
			FTableTreeViewCommands::Get().Command_ExpandSubtree,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FInsightsCoreStyle::GetStyleSetName(), "Icons.ExpandSelection")
		);

		MenuBuilder.AddMenuEntry
		(
			FTableTreeViewCommands::Get().Command_ExpandCriticalPath,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FInsightsCoreStyle::GetStyleSetName(), "Icons.ExpandSelection")
		);

		MenuBuilder.AddMenuEntry
		(
			FTableTreeViewCommands::Get().Command_CollapseSubtree,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FInsightsCoreStyle::GetStyleSetName(), "Icons.CollapseSelection")
		);
	}

	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Misc", LOCTEXT("ContextMenu_Section_Misc", "Miscellaneous"));
	{
		MenuBuilder.AddMenuEntry
		(
			FTableTreeViewCommands::Get().Command_CopyToClipboard,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCommands.Copy")
		);

		MenuBuilder.AddMenuEntry
		(
			FTableTreeViewCommands::Get().Command_CopyColumnToClipboard,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCommands.Copy")
		);

		MenuBuilder.AddMenuEntry
		(
			FTableTreeViewCommands::Get().Command_CopyColumnTooltipToClipboard,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCommands.Copy")
		);

		MenuBuilder.AddSubMenu
		(
			LOCTEXT("ContextMenu_Export_SubMenu", "Export"),
			LOCTEXT("ContextMenu_Export_SubMenu_Desc", "Exports to a file."),
			FNewMenuDelegate::CreateSP(this, &STableTreeView::TreeView_BuildExportMenu),
			false,
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Save")
		);
	}
	MenuBuilder.EndSection();

	ExtendMenu(MenuBuilder);

	MenuBuilder.PopExtender();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::TreeView_BuildSortByMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("SortColumn", LOCTEXT("ContextMenu_Section_SortColumn", "Sort Column"));

	//TODO: for (Sorting : AvailableSortings)
	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		const FTableColumn& Column = ColumnRef.Get();
		if (Column.IsVisible() && Column.CanBeSorted())
		{
			FUIAction Action_SortByColumn
			(
				FExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_SortByColumn_Execute, Column.GetId()),
				FCanExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_SortByColumn_CanExecute, Column.GetId()),
				FIsActionChecked::CreateSP(this, &STableTreeView::ContextMenu_SortByColumn_IsChecked, Column.GetId())
			);
			MenuBuilder.AddMenuEntry
			(
				Column.GetTitleName(),
				Column.GetDescription(),
				FSlateIcon(),
				Action_SortByColumn,
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);
		}
	}

	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("SortMode", LOCTEXT("ContextMenu_Section_SortMode", "Sort Mode"));
	{
		FUIAction Action_SortAscending
		(
			FExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_SortMode_Execute, EColumnSortMode::Ascending),
			FCanExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_SortMode_CanExecute, EColumnSortMode::Ascending),
			FIsActionChecked::CreateSP(this, &STableTreeView::ContextMenu_SortMode_IsChecked, EColumnSortMode::Ascending)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_SortAscending", "Sort Ascending"),
			LOCTEXT("ContextMenu_SortAscending_Desc", "Sorts ascending."),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SortUp"),
			Action_SortAscending,
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		FUIAction Action_SortDescending
		(
			FExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_SortMode_Execute, EColumnSortMode::Descending),
			FCanExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_SortMode_CanExecute, EColumnSortMode::Descending),
			FIsActionChecked::CreateSP(this, &STableTreeView::ContextMenu_SortMode_IsChecked, EColumnSortMode::Descending)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_SortDescending", "Sort Descending"),
			LOCTEXT("ContextMenu_SortDescending_Desc", "Sorts descending."),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SortDown"),
			Action_SortDescending,
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::TreeView_BuildViewColumnMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("Columns", LOCTEXT("ContextMenu_Section_Columns", "Columns"));

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		const FTableColumn& Column = ColumnRef.Get();

		FUIAction Action_ToggleColumn
		(
			FExecuteAction::CreateSP(this, &STableTreeView::ToggleColumnVisibility, Column.GetId()),
			FCanExecuteAction::CreateSP(this, &STableTreeView::CanToggleColumnVisibility, Column.GetId()),
			FIsActionChecked::CreateSP(this, &STableTreeView::IsColumnVisible, Column.GetId())
		);
		MenuBuilder.AddMenuEntry
		(
			Column.GetTitleName(),
			Column.GetDescription(),
			FSlateIcon(),
			Action_ToggleColumn,
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}

	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::TreeView_BuildExportMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.SetSearchable(false);

	MenuBuilder.BeginSection("Export");
	{
		MenuBuilder.AddMenuEntry
		(
			FTableTreeViewCommands::Get().Command_ExportToFile,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Save")
		);

		MenuBuilder.AddMenuEntry
		(
			FTableTreeViewCommands::Get().Command_ExportEntireTreeToFile,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Save")
		);

		MenuBuilder.AddMenuEntry
		(
			FTableTreeViewCommands::Get().Command_ExportEntireTreeToFileExceptLeaves,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Save")
		);
	}
	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::InitializeAndShowHeaderColumns()
{
	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		FTableColumn& Column = ColumnRef.Get();
		if (Column.ShouldBeVisible())
		{
			ShowColumn(Column);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::RebuildColumns()
{
	TreeViewHeaderRow->ClearColumns();
	InitializeAndShowHeaderColumns();

	PreChangeGroupings();
	CreateGroupings();
	PostChangeGroupings();

	CreateSortings();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STableTreeView::GetColumnHeaderText(const FName ColumnId) const
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return Column.GetShortName();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STableTreeView::TreeViewHeaderRow_GenerateColumnMenu(const FTableColumn& Column)
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, NULL);

	MenuBuilder.BeginSection("Sorting", LOCTEXT("ContextMenu_Section_Sorting", "Sorting"));
	{
		if (Column.CanBeSorted())
		{
			FUIAction Action_SortAscending
			(
				FExecuteAction::CreateSP(this, &STableTreeView::HeaderMenu_SortMode_Execute, Column.GetId(), EColumnSortMode::Ascending),
				FCanExecuteAction::CreateSP(this, &STableTreeView::HeaderMenu_SortMode_CanExecute, Column.GetId(), EColumnSortMode::Ascending),
				FIsActionChecked::CreateSP(this, &STableTreeView::HeaderMenu_SortMode_IsChecked, Column.GetId(), EColumnSortMode::Ascending)
			);
			MenuBuilder.AddMenuEntry
			(
				FText::Format(LOCTEXT("ContextMenu_SortAscending_Fmt", "Sort Ascending (by {0})"), Column.GetTitleName()),
				FText::Format(LOCTEXT("ContextMenu_SortAscending_Desc_Fmt", "Sorts ascending by {0}."), Column.GetTitleName()),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SortUp"),
				Action_SortAscending,
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);

			FUIAction Action_SortDescending
			(
				FExecuteAction::CreateSP(this, &STableTreeView::HeaderMenu_SortMode_Execute, Column.GetId(), EColumnSortMode::Descending),
				FCanExecuteAction::CreateSP(this, &STableTreeView::HeaderMenu_SortMode_CanExecute, Column.GetId(), EColumnSortMode::Descending),
				FIsActionChecked::CreateSP(this, &STableTreeView::HeaderMenu_SortMode_IsChecked, Column.GetId(), EColumnSortMode::Descending)
			);
			MenuBuilder.AddMenuEntry
			(
				FText::Format(LOCTEXT("ContextMenu_SortDescending_Fmt", "Sort Descending (by {0})"), Column.GetTitleName()),
				FText::Format(LOCTEXT("ContextMenu_SortDescending_Desc_Fmt", "Sorts descending by {0}."), Column.GetTitleName()),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SortDown"),
				Action_SortDescending,
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);
		}

		MenuBuilder.AddSubMenu
		(
			LOCTEXT("ContextMenu_SortBy_SubMenu", "Sort By"),
			LOCTEXT("ContextMenu_SortBy_SubMenu_Desc", "Sorts by a column."),
			FNewMenuDelegate::CreateSP(this, &STableTreeView::TreeView_BuildSortByMenu),
			false,
			FSlateIcon(FInsightsCoreStyle::GetStyleSetName(), "Icons.SortBy")
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("ColumnVisibility", LOCTEXT("ContextMenu_Section_ColumnVisibility", "Column Visibility"));
	{
		if (Column.CanBeHidden())
		{
			FUIAction Action_HideColumn
			(
				FExecuteAction::CreateSP(this, &STableTreeView::HideColumn, Column.GetId()),
				FCanExecuteAction::CreateSP(this, &STableTreeView::CanHideColumn, Column.GetId())
			);
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_HideColumn", "Hide"),
				LOCTEXT("ContextMenu_HideColumn_Desc", "Hides the selected column."),
				FSlateIcon(),
				Action_HideColumn,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}

		MenuBuilder.AddSubMenu
		(
			LOCTEXT("ContextMenu_ViewColumn_SubMenu", "View Column"),
			LOCTEXT("ContextMenu_ViewColumn_SubMenu_Desc", "Hides or shows columns."),
			FNewMenuDelegate::CreateSP(this, &STableTreeView::TreeView_BuildViewColumnMenu),
			false,
			FSlateIcon(FInsightsCoreStyle::GetStyleSetName(), "Icons.ViewColumn")
		);

		FUIAction Action_ShowAllColumns
		(
			FExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_ShowAllColumns_Execute),
			FCanExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_ShowAllColumns_CanExecute)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_ShowAllColumns", "Show All Columns"),
			LOCTEXT("ContextMenu_ShowAllColumns_Desc", "Resets tree view to show all columns."),
			FSlateIcon(FInsightsCoreStyle::GetStyleSetName(), "Icons.ResetColumn"),
			Action_ShowAllColumns,
			NAME_None,
			EUserInterfaceActionType::Button
		);

		FUIAction Action_ResetColumns
		(
			FExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_ResetColumns_Execute),
			FCanExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_ResetColumns_CanExecute)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_ResetColumns", "Reset Columns to Default"),
			LOCTEXT("ContextMenu_ResetColumns_Desc", "Resets columns to default."),
			FSlateIcon(FInsightsCoreStyle::GetStyleSetName(), "Icons.ResetColumn"),
			Action_ResetColumns,
			NAME_None,
			EUserInterfaceActionType::Button
		);

		FUIAction Action_HideAllColumns
		(
			FExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_HideAllColumns_Execute),
			FCanExecuteAction::CreateSP(this, &STableTreeView::ContextMenu_HideAllColumns_CanExecute)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_HideAllColumns", "Hide All Columns"),
			LOCTEXT("ContextMenu_HideAllColumns_Desc", "Resets tree view to hide all columns (except hierarchy)."),
			FSlateIcon(FInsightsCoreStyle::GetStyleSetName(), "Icons.ResetColumn"),
			Action_HideAllColumns,
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bRunInAsyncMode && bIsUpdateRunning && !bIsCloseScheduled)
	{
		if (DispatchEvent.IsValid() && !DispatchEvent->IsComplete())
		{
			// We wait for the TreeView to be refreshed before dispatching the tasks.
			// This should make the TreeView release all of it's shared pointers to nodes to prevent
			// the TreeView (MainThread) and the tasks from accessing the non-thread safe shared pointers at the same time.
			if (!TreeView->IsPendingRefresh())
			{
				DispatchEvent->DispatchSubsequents();
			}
		}
		else
		{
			check(InProgressAsyncOperationEvent.IsValid());
			if (InProgressAsyncOperationEvent->IsComplete())
			{
				OnPostAsyncUpdate();
				StartPendingAsyncOperations();
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::UpdateTree()
{
	OnNodeFilteringChanged();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Node Filtering
////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::InitNodeFiltering()
{
	// Filter Configurator is created when the Filter Configurator button is pressed.
	//FilterConfigurator = nullptr;
	//CurrentAsyncOpFilterConfigurator = nullptr;
	//OnFilterChangesCommittedHandle = nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::OnNodeFilteringChanged()
{
	if (!bRunInAsyncMode)
	{
		ApplyNodeFiltering();
		ApplyGrouping();
		ApplySorting();
		ApplyHierarchyFiltering();
	}
	else
	{
		AddPendingAsyncOperation(EAsyncOperationType::NodeFiltering);
		if (!bIsUpdateRunning)
		{
			ScheduleNodeFilteringAsyncOperation();
		}
		else
		{
			CancelCurrentAsyncOp();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::ScheduleNodeFilteringAsyncOperationIfNeeded()
{
	if (HasPendingAsyncOperation(EAsyncOperationType::NodeFiltering) ||
		HasInProgressAsyncOperation(EAsyncOperationType::NodeFiltering) ||
		(FilterConfigurator.IsValid() &&
		 CurrentAsyncOpFilterConfigurator &&
		 *FilterConfigurator != *CurrentAsyncOpFilterConfigurator))
	{
		ScheduleNodeFilteringAsyncOperation();
		return true;
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::ScheduleNodeFilteringAsyncOperation()
{
	OnPreAsyncUpdate();

	ClearPendingAsyncOperation(EAsyncOperationType::NodeFiltering);

	FGraphEventRef CompletedEvent = StartNodeFilteringTask();
	CompletedEvent = StartGroupingTask(CompletedEvent);
	CompletedEvent = StartSortingTask(CompletedEvent);
	InProgressAsyncOperationEvent = StartHierarchyFilteringTask(CompletedEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FGraphEventRef STableTreeView::StartNodeFilteringTask(FGraphEventRef Prerequisite)
{
	AddInProgressAsyncOperation(EAsyncOperationType::NodeFiltering);

	if (FilterConfigurator.IsValid() && CurrentAsyncOpFilterConfigurator)
	{
		*CurrentAsyncOpFilterConfigurator = *FilterConfigurator;
	}

	FGraphEventArray Prerequisites;
	if (Prerequisite.IsValid())
	{
		Prerequisites.Add(Prerequisite);
	}
	else
	{
		Prerequisites.Add(DispatchEvent);
	}

	return TGraphTask<FTableTreeViewNodeFilteringAsyncTask>::CreateTask(&Prerequisites).ConstructAndDispatchWhenReady(this);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

// Note: This function is called from a task thread!
void STableTreeView::ApplyNodeFiltering()
{
	FStopwatch Stopwatch;
	Stopwatch.Start();

	FFilterConfigurator* FilterConfiguratorToUse = bRunInAsyncMode ? CurrentAsyncOpFilterConfigurator : FilterConfigurator.Get();

	// TableRowNodes --> FilteredNodes / FilteredNodesPtr
	FilteredNodes.Reset();
	const bool bHasValidFilterConfigurator = FilterConfiguratorToUse && !FilterConfiguratorToUse->IsEmpty();
	const bool bHasCustomNodeFilter = HasCustomNodeFilter();
	if (bHasValidFilterConfigurator || bHasCustomNodeFilter)
	{
		for (const FTableTreeNodePtr& NodePtr : TableRowNodes)
		{
			if (AsyncOperationProgress.ShouldCancelAsyncOp())
			{
				break;
			}
			if ((!bHasValidFilterConfigurator || FilterNode(*FilterConfiguratorToUse, *NodePtr)) &&
				(!bHasCustomNodeFilter || FilterNodeCustom(*NodePtr)))
			{
				FilteredNodes.Add(NodePtr);
			}
		}
		if (FilteredNodes.Num() != TableRowNodes.Num())
		{
			FilteredNodesPtr = &FilteredNodes;
		}
		else
		{
			FilteredNodes.Reset();
			FilteredNodesPtr = &TableRowNodes; // no filter; includes all nodes
		}
	}
	else
	{
		FilteredNodesPtr = &TableRowNodes; // no filter; includes all nodes
	}

	Stopwatch.Stop();
	const double FilteringTime = Stopwatch.GetAccumulatedTime();
	if (FilteringTime > 0.1)
	{
		UE_LOGF(LogInsights, Log, "[Tree - %ls] Node filtering completed in %.3fs.", *Table->GetDisplayName().ToString(), FilteringTime);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::FilterNode(const FFilterConfigurator& InFilterConfigurator, const FTableTreeNode& InNode) const
{
	UpdateFilterContext(InFilterConfigurator, InNode);
	return InFilterConfigurator.ApplyFilters(FilterContext);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::InitFilterConfigurator(FFilterConfigurator& InOutFilterConfigurator)
{
	for (const TSharedRef<FTableColumn>& Column : Table->GetColumns())
	{
		if (!Column->CanBeFiltered())
		{
			continue;
		}

		switch (Column->GetDataType())
		{
			case ETableCellDataType::Int64:
			{
				TSharedRef<FFilter> Filter = MakeShared<FFilter>(
					Column->GetIndex(),
					Column->GetTitleName(),
					Column->GetDescription(),
					EFilterDataType::Int64,
					Column->GetValueConverter(),
					FFilterService::Get()->GetIntegerOperators()
				);
				InOutFilterConfigurator.Add(Filter);
				FilterContext.AddFilterData<int64>(Column->GetIndex(), 0);
				break;
			}

			case ETableCellDataType::Double:
			{
				TSharedRef<FFilter> Filter = MakeShared<FFilter>(
					Column->GetIndex(),
					Column->GetTitleName(),
					Column->GetDescription(),
					EFilterDataType::Double,
					Column->GetValueConverter(),
					FFilterService::Get()->GetDoubleOperators());
				InOutFilterConfigurator.Add(Filter);
				FilterContext.AddFilterData<double>(Column->GetIndex(), 0.0);
				break;
			}

			case ETableCellDataType::CString:
			case ETableCellDataType::Text:
			case ETableCellDataType::Custom:
			{
				if (!Column->IsHierarchy())
				{
					TSharedRef<FFilter> Filter = MakeShared<FFilter>(
						Column->GetIndex(),
						Column->GetTitleName(),
						Column->GetDescription(),
						EFilterDataType::String,
						Column->GetValueConverter(),
						FFilterService::Get()->GetStringOperators());
					InOutFilterConfigurator.Add(Filter);
					FilterContext.AddFilterData<FString>(Column->GetIndex(), FString());
				}
				break;
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::UpdateFilterContext(const FFilterConfigurator& InFilterConfigurator, const FTableTreeNode& InNode) const
{
	for (const TSharedRef<FTableColumn>& Column : Table->GetColumns())
	{
		if (!Column->CanBeFiltered() ||
			!InFilterConfigurator.IsKeyUsed(Column->GetIndex()) ||
			!Column->GetValue(InNode).IsSet())
		{
			continue;
		}

		switch (Column->GetDataType())
		{
			case ETableCellDataType::Int64:
			{
				FilterContext.SetFilterData<int64>(Column->GetIndex(), Column->GetValue(InNode)->AsInt64());
				break;
			}

			case ETableCellDataType::Double:
			{
				FilterContext.SetFilterData<double>(Column->GetIndex(), Column->GetValue(InNode)->AsDouble());
				break;
			}

			case ETableCellDataType::CString:
			case ETableCellDataType::Text:
			case ETableCellDataType::Custom:
			{
				if (!Column->IsHierarchy())
				{
					FilterContext.SetFilterData<FString>(Column->GetIndex(), Column->GetValue(InNode)->AsString());
				}
				break;
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> STableTreeView::ConstructFilterConfiguratorButton()
{
	return SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.ToolTipText(LOCTEXT("FilterConfiguratorBtn_ToolTip", "Opens the filter configurator window."))
		.IsEnabled_Lambda([this]()
			{
				return Table.IsValid() && Table->IsValid();
			})
		.OnClicked(this, &STableTreeView::FilterConfigurator_OnClicked)
		[
			SNew(SImage)
			.Image(FInsightsCoreStyle::GetBrush("Icons.ClassicFilterConfig"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STableTreeView::FilterConfigurator_OnClicked()
{
	if (!FilterConfigurator.IsValid())
	{
		FilterConfigurator = MakeShared<FFilterConfigurator>();
		InitFilterConfigurator(*FilterConfigurator);
		OnFilterChangesCommittedHandle = FilterConfigurator->GetOnChangesCommittedEvent().AddSP(this, &STableTreeView::OnNodeFilteringChanged);
		CurrentAsyncOpFilterConfigurator = new FFilterConfigurator(*FilterConfigurator);
	}

	FFilterService::Get()->CreateFilterConfiguratorWidget(FilterConfigurator, SharedThis(this));

	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Hierarchy Filtering
////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::InitHierarchyFiltering()
{
	// The text filter editable on UI thread.
	TextFilter = MakeShared<FTableTreeNodeTextFilter>(FTableTreeNodeTextFilter::FItemToStringArray::CreateStatic(&STableTreeView::HandleItemToStringArray));

	if (bRunInAsyncMode)
	{
		// A copy of the TextFilter used to actually apply the filtering in an async task.
		CurrentAsyncOpTextFilter = MakeShared<FTableTreeNodeTextFilter>(FTableTreeNodeTextFilter::FItemToStringArray::CreateStatic(&STableTreeView::HandleItemToStringArray));
	}

	Filters = MakeShared<FTableTreeNodeFilterCollection>();
	if (bRunInAsyncMode)
	{
		Filters->Add(CurrentAsyncOpTextFilter);
	}
	else
	{
		Filters->Add(TextFilter);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::HandleItemToStringArray(const FTableTreeNodePtr& FTableTreeNodePtr, TArray<FString>& OutSearchStrings)
{
	OutSearchStrings.Add(FTableTreeNodePtr->GetDisplayName().ToString());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::OnHierarchyFilteringChanged()
{
	if (!bRunInAsyncMode)
	{
		ApplyHierarchyFiltering();
	}
	else
	{
		AddPendingAsyncOperation(EAsyncOperationType::HierarchyFiltering);
		if (!bIsUpdateRunning)
		{
			ScheduleHierarchyFilteringAsyncOperation();
		}
		else
		{
			CancelCurrentAsyncOp();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::ScheduleHierarchyFilteringAsyncOperationIfNeeded()
{
	if (HasPendingAsyncOperation(EAsyncOperationType::HierarchyFiltering) ||
		HasInProgressAsyncOperation(EAsyncOperationType::HierarchyFiltering) ||
		TextFilter->GetRawFilterText().CompareTo(CurrentAsyncOpTextFilter->GetRawFilterText()) != 0)
	{
		ScheduleHierarchyFilteringAsyncOperation();
		return true;
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::ScheduleHierarchyFilteringAsyncOperation()
{
	OnPreAsyncUpdate();

	ClearPendingAsyncOperation(EAsyncOperationType::HierarchyFiltering);

	InProgressAsyncOperationEvent = StartHierarchyFilteringTask();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FGraphEventRef STableTreeView::StartHierarchyFilteringTask(FGraphEventRef Prerequisite)
{
	AddInProgressAsyncOperation(EAsyncOperationType::HierarchyFiltering);

	CurrentAsyncOpTextFilter->SetRawFilterText(TextFilter->GetRawFilterText());

	FGraphEventArray Prerequisites;
	if (Prerequisite.IsValid())
	{
		Prerequisites.Add(Prerequisite);
	}
	else
	{
		Prerequisites.Add(DispatchEvent);
	}

	return TGraphTask<FTableTreeViewHierarchyFilteringAsyncTask>::CreateTask(&Prerequisites).ConstructAndDispatchWhenReady(this);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

// Note: This function is called from a task thread!
void STableTreeView::ApplyHierarchyFiltering()
{
	FStopwatch Stopwatch;
	Stopwatch.Start();

	const bool bIsEmptyFilter = bRunInAsyncMode ? CurrentAsyncOpTextFilter->GetRawFilterText().IsEmpty() : TextFilter->GetRawFilterText().IsEmpty();
	if (bIsEmptyFilter)
	{
		ApplyEmptyHierarchyFilteringRec(Root);
	}
	else
	{
		ApplyHierarchyFilteringRec(Root);
	}

	// The Root node is always hidden. The tree shows the filtered children of the Root node.
	const TArray<FBaseTreeNodePtr>& RootChildren = Root->GetFilteredChildren();
	FilteredGroupNodes.Reset(RootChildren.Num());
	for (const FBaseTreeNodePtr& ChildNodePtr : RootChildren)
	{
		if (ChildNodePtr->IsGroup())
		{
			FilteredGroupNodes.Add(StaticCastSharedPtr<FTableTreeNode>(ChildNodePtr));
		}
	}

	// Cannot call TreeView functions from other threads than MainThread and SlateThread.
	if (!bRunInAsyncMode)
	{
		Root->SetExpansion(true);
		ApplyNodeExpansion(Root);

		TreeView->RequestTreeRefresh();
	}

	UpdateBannerText();

	Stopwatch.Stop();
	const double FilteringTime = Stopwatch.GetAccumulatedTime();
	if (FilteringTime > 0.1)
	{
		UE_LOGF(LogInsights, Log, "[Tree - %ls] Hierarchy filtering completed in %.3fs.", *Table->GetDisplayName().ToString(), FilteringTime);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::ApplyEmptyHierarchyFilteringRec(FTableTreeNodePtr NodePtr)
{
	if (NodePtr->IsGroup())
	{
		// If a group node passes the filter, all child nodes will be shown.
		MakeSubtreeVisible(NodePtr, true);
	}
	else
	{
		NodePtr->SetIsFiltered(false);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::ApplyHierarchyFilteringRec(FTableTreeNodePtr NodePtr)
{
	bool bIsNodeVisible = Filters->PassesAllFilters(NodePtr);

	if (NodePtr->IsGroup())
	{
		// If a group node passes the filter, all child nodes will be shown.
		if (bIsNodeVisible)
		{
			MakeSubtreeVisible(NodePtr, false);
			return true;
		}

		const TArray<FBaseTreeNodePtr>& GroupChildren = NodePtr->GetChildren();

		NodePtr->ClearFilteredChildren(0);

		int32 NumVisibleChildren = 0;
		for (const FBaseTreeNodePtr& GroupChildNodePtr : GroupChildren)
		{
			if (AsyncOperationProgress.ShouldCancelAsyncOp())
			{
				break;
			}

			// Add a child.
			const FTableTreeNodePtr& ChildNodePtr = StaticCastSharedPtr<FTableTreeNode>(GroupChildNodePtr);
			if (ApplyHierarchyFilteringRec(ChildNodePtr))
			{
				NodePtr->AddFilteredChild(ChildNodePtr);
				NumVisibleChildren++;
			}
		}

		const bool bIsGroupNodeVisible = bIsNodeVisible || NumVisibleChildren > 0;
		if (bIsGroupNodeVisible)
		{
			NodePtr->SetExpansion(true);
			if (!bRunInAsyncMode)
			{
				TreeView->SetItemExpansion(NodePtr, true);
			}
		}
		return bIsGroupNodeVisible;
	}
	else
	{
		NodePtr->SetIsFiltered(!bIsNodeVisible);
		return bIsNodeVisible;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::MakeSubtreeVisible(FTableTreeNodePtr NodePtr, bool bFilterIsEmpty)
{
	bool bPassesNonEmptyFilter = !bFilterIsEmpty && Filters->PassesAllFilters(NodePtr);
	if (NodePtr->IsGroup())
	{
		const TArray<FBaseTreeNodePtr>& GroupChildren = NodePtr->GetChildren();

		NodePtr->ClearFilteredChildren(bFilterIsEmpty ? GroupChildren.Num() : 0);

		bool bShouldExpand = NodePtr->IsExpanded();
		for (const FBaseTreeNodePtr& GroupChildNodePtr : GroupChildren)
		{
			if (AsyncOperationProgress.ShouldCancelAsyncOp())
			{
				break;
			}

			const FTableTreeNodePtr& ChildNodePtr = StaticCastSharedPtr<FTableTreeNode>(GroupChildNodePtr);
			bShouldExpand |= MakeSubtreeVisible(ChildNodePtr, bFilterIsEmpty);
			NodePtr->AddFilteredChild(ChildNodePtr);
			NodePtr->SetExpansion(bShouldExpand);
			if (!bRunInAsyncMode)
			{
				TreeView->SetItemExpansion(NodePtr, bShouldExpand);
			}
		}

		return bShouldExpand || bPassesNonEmptyFilter;
	}

	NodePtr->SetIsFiltered(false);
	return bPassesNonEmptyFilter;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// SearchBox
////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> STableTreeView::ConstructSearchBox()
{
	SAssignNew(SearchBox, SSearchBox)
		.HintText(LOCTEXT("SearchBox_Hint", "Search"))
		.ToolTipText(LOCTEXT("SearchBox_ToolTip", "Type here to search the tree hierarchy by item or group name."))
		.OnTextChanged(this, &STableTreeView::SearchBox_OnTextChanged);

	return SearchBox.ToSharedRef();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::SearchBox_OnTextChanged(const FText& InFilterText)
{
	TextFilter->SetRawFilterText(InFilterText);
	SearchBox->SetError(TextFilter->GetFilterErrorText());
	OnHierarchyFilteringChanged();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// TreeView
////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::TreeView_Refresh()
{
	if (TreeView.IsValid())
	{
		TreeView->RequestTreeRefresh();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::TreeView_OnSelectionChanged(FTableTreeNodePtr SelectedItem, ESelectInfo::Type SelectInfo)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::TreeView_OnExpansionChanged(FTableTreeNodePtr TreeNode, bool bShouldBeExpanded)
{
	if (TreeNode->IsExpanded() != bShouldBeExpanded)
	{
		TreeNode->SetExpansion(bShouldBeExpanded);
		if (bShouldBeExpanded)
		{
			ApplyNodeExpansion(TreeNode);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::TreeView_OnGetChildren(FTableTreeNodePtr InParent, TArray<FTableTreeNodePtr>& OutChildren)
{
	if (InParent->OnLazyCreateChildren(SharedThis(this)))
	{
		// Node filtering does not apply to lazy expanded nodes.

		// Apply grouping only if the node specifies the grouping that created the node.
		// In this case, it will apply groupings after the identified one.
		const FTreeNodeGrouping* AuthorGrouping = InParent->GetAuthorGrouping();
		if (AuthorGrouping)
		{
			TArray<TSharedPtr<FTreeNodeGrouping>> Groupings;

			bool bFound = false;
			for (TSharedPtr<FTreeNodeGrouping> Grouping : CurrentGroupings)
			{
				if (bFound)
				{
					Groupings.Add(Grouping);
				}
				else
				{
					if (Grouping.Get() == AuthorGrouping)
					{
						bFound = true;
					}
				}
			}

			if (Groupings.Num() > 0)
			{
				// Extract children nodes.
				TArray<FTableTreeNodePtr> ChildNodes;
				InParent->SwapChildrenFast(reinterpret_cast<TArray<FBaseTreeNodePtr>&>(ChildNodes));

				GroupNodesRec(ChildNodes, *InParent, 0, Groupings);
			}
		}

		// Update aggregation.
		UpdateAggregatedValuesRec(*InParent);
		FTableTreeNodePtr Node = StaticCastSharedPtr<FTableTreeNode>(InParent->GetParent());
		while (Node)
		{
			UpdateAggregatedValuesSingleNode(*Node);
			Node = StaticCastSharedPtr<FTableTreeNode>(Node->GetParent());
		}

		// Update sorting.
		if (CurrentSorter.IsValid())
		{
			SortTreeNodesRec(*InParent, *CurrentSorter, ColumnSortMode);
		}

		// The hierarchy filtering is ignored for lazy expanded nodes.
		InParent->ResetFilteredChildrenRec();
	}

	const TArray<FBaseTreeNodePtr>& FilteredChildren = InParent->GetFilteredChildren();
	for (const FBaseTreeNodePtr& NodePtr : FilteredChildren)
	{
		OutChildren.Add(StaticCastSharedPtr<FTableTreeNode>(NodePtr));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::TreeView_OnMouseButtonDoubleClick(FTableTreeNodePtr NodePtr)
{
	if (NodePtr->IsGroup())
	{
		const bool bIsGroupExpanded = TreeView->IsItemExpanded(NodePtr);
		TreeView->SetItemExpansion(NodePtr, !bIsGroupExpanded);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Tree View's Table Row
////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<ITableRow> STableTreeView::TreeView_OnGenerateRow(FTableTreeNodePtr NodePtr, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<ITableRow> TableRow =
		SNew(STableTreeViewRow, OwnerTable)
		.OnShouldBeEnabled(this, &STableTreeView::TableRow_ShouldBeEnabled)
		.OnIsColumnVisible(this, &STableTreeView::IsColumnVisible)
		.OnSetHoveredCell(this, &STableTreeView::TableRow_SetHoveredCell)
		.OnGetColumnOutlineHAlignmentDelegate(this, &STableTreeView::TableRow_GetColumnOutlineHAlignment)
		.HighlightText(this, &STableTreeView::TableRow_GetHighlightText)
		.TablePtr(Table)
		.TableTreeNodePtr(NodePtr);

	return TableRow;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::TableRow_ShouldBeEnabled(FTableTreeNodePtr NodePtr) const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::TableRow_SetHoveredCell(TSharedPtr<FTable> InTablePtr, TSharedPtr<FTableColumn> InColumnPtr, FTableTreeNodePtr InNodePtr)
{
	const bool bIsAnyMenusVisible = FSlateApplication::Get().AnyMenusVisible();
	if (!HasMouseCapture() && !bIsAnyMenusVisible)
	{
		HoveredColumnId = InColumnPtr ? InColumnPtr->GetId() : FName();
		HoveredNodePtr = InNodePtr;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EHorizontalAlignment STableTreeView::TableRow_GetColumnOutlineHAlignment(const FName ColumnId) const
{
	const TIndirectArray<SHeaderRow::FColumn>& Columns = TreeViewHeaderRow->GetColumns();

	// First column
	if (Columns.Num() == 0 ||
		Columns[0].ColumnId == ColumnId)
	{
		return HAlign_Left;
	}
	// Last column
	else if (Columns[Columns.Num() - 1].ColumnId == ColumnId)
	{
		return HAlign_Right;
	}
	// Middle columns
	{
		return HAlign_Center;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STableTreeView::TableRow_GetHighlightText() const
{
	return SearchBox->GetText();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Grouping
////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::OnGroupingChanged()
{
	if (!bRunInAsyncMode)
	{
		ApplyGrouping();
		ApplySorting();
		ApplyHierarchyFiltering();
	}
	else
	{
		AddPendingAsyncOperation(EAsyncOperationType::Grouping);
		if (!bIsUpdateRunning)
		{
			ScheduleGroupingAsyncOperation();
		}
		else
		{
			CancelCurrentAsyncOp();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::ScheduleGroupingAsyncOperationIfNeeded()
{
	bool bIsGroupingNeeded =
		HasPendingAsyncOperation(EAsyncOperationType::Grouping) ||
		HasInProgressAsyncOperation(EAsyncOperationType::Grouping);

	if (!bIsGroupingNeeded &&
		CurrentGroupings.Num() != CurrentAsyncOpGroupings.Num())
	{
		bIsGroupingNeeded = true;
	}

	if (!bIsGroupingNeeded)
	{
		for (int32 Index = 0; Index < CurrentGroupings.Num(); ++Index)
		{
			if (CurrentGroupings[Index] != CurrentAsyncOpGroupings[Index])
			{
				bIsGroupingNeeded = true;
				break;
			}
		}
	}

	if (bIsGroupingNeeded)
	{
		ScheduleGroupingAsyncOperation();
		return true;
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::ScheduleGroupingAsyncOperation()
{
	OnPreAsyncUpdate();

	ClearPendingAsyncOperation(EAsyncOperationType::Grouping);

	FGraphEventRef CompletedEvent = StartGroupingTask();
	CompletedEvent = StartSortingTask(CompletedEvent);
	InProgressAsyncOperationEvent = StartHierarchyFilteringTask(CompletedEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FGraphEventRef STableTreeView::StartGroupingTask(FGraphEventRef Prerequisite)
{
	AddInProgressAsyncOperation(EAsyncOperationType::Grouping);

	CurrentAsyncOpGroupings = CurrentGroupings;

	FGraphEventArray Prerequisites;
	if (Prerequisite.IsValid())
	{
		Prerequisites.Add(Prerequisite);
	}
	else
	{
		Prerequisites.Add(DispatchEvent);
	}

	return TGraphTask<FTableTreeViewGroupingAsyncTask>::CreateTask(&Prerequisites).ConstructAndDispatchWhenReady(this, &CurrentAsyncOpGroupings);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::ApplyGrouping()
{
	CreateGroups(CurrentGroupings);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

// Note: This function is called from a task thread!
void STableTreeView::CreateGroups(const TArray<TSharedPtr<FTreeNodeGrouping>>& Groupings)
{
	if (FilteredNodesPtr->Num() == 0)
	{
		Root->ClearChildren();
		return;
	}

	FStopwatch Stopwatch;
	Stopwatch.Start();

	if (Groupings.Num() > 0)
	{
		GroupNodesRec(*FilteredNodesPtr, *Root, 0, Groupings);
	}

	Stopwatch.Stop();
	const double GroupingTime = Stopwatch.GetAccumulatedTime();
	if (GroupingTime > 0.1)
	{
		UE_LOGF(LogInsights, Log, "[Tree - %ls] Grouping completed in %.3fs.", *Table->GetDisplayName().ToString(), GroupingTime);
	}

	UpdateAggregatedValuesRec(*Root);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::GroupNodesRec(const TArray<FTableTreeNodePtr>& Nodes, FTableTreeNode& ParentGroup, int32 GroupingDepth, const TArray<TSharedPtr<FTreeNodeGrouping>>& Groupings)
{
	if (AsyncOperationProgress.ShouldCancelAsyncOp())
	{
		return;
	}

	if (!ensure(GroupingDepth < Groupings.Num()))
	{
		return;
	}

	TSet<FBaseTreeNode*> OldGroupNodes;

	// Apply current grouping recursively to the (old) group nodes.
	for (FBaseTreeNodePtr NodePtr : Nodes)
	{
		if (NodePtr->IsGroup() && NodePtr->Is<FTableTreeNode>())
		{
			OldGroupNodes.Add(NodePtr.Get());

			FTableTreeNode& Group = NodePtr->As<FTableTreeNode>();

			// Extract child nodes.
			TArray<FTableTreeNodePtr> ChildNodes;
			Group.SwapChildrenFast(reinterpret_cast<TArray<FBaseTreeNodePtr>&>(ChildNodes));

			GroupNodesRec(ChildNodes, Group, GroupingDepth, Groupings);
		}
	}

	ParentGroup.ClearChildren();

	// Group the current list of nodes.
	FTreeNodeGrouping& Grouping = *Groupings[GroupingDepth];
	Grouping.GroupNodes(Nodes, ParentGroup, Table, AsyncOperationProgress);

	if (AsyncOperationProgress.ShouldCancelAsyncOp())
	{
		return;
	}

	if (GroupingDepth < Groupings.Num() - 1)
	{
		// Apply the next grouping recursively to the new group nodes.
		for (FBaseTreeNodePtr GroupPtr : ParentGroup.GetChildren())
		{
			if (GroupPtr->IsGroup() && GroupPtr->Is<FTableTreeNode>())
			{
				if (!OldGroupNodes.Contains(GroupPtr.Get()))
				{
					FTableTreeNode& Group = GroupPtr->As<FTableTreeNode>();

					// Extract child nodes.
					TArray<FTableTreeNodePtr> ChildNodes;
					Group.SwapChildrenFast(reinterpret_cast<TArray<FBaseTreeNodePtr>&>(ChildNodes));

					GroupNodesRec(ChildNodes, Group, GroupingDepth + 1, Groupings);
				}
			}
		}
	}

	if (!bRunInAsyncMode)
	{
		// Expand group nodes.
		for (FBaseTreeNodePtr GroupPtr : ParentGroup.GetChildren())
		{
			if (GroupPtr->IsGroup() && GroupPtr->Is<FTableTreeNode>())
			{
				FTableTreeNodePtr TableTreeGroupPtr = StaticCastSharedPtr<FTableTreeNode>(GroupPtr);
				TreeView->SetItemExpansion(TableTreeGroupPtr, TableTreeGroupPtr->IsExpanded());
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::CreateGroupings()
{
	AvailableGroupings.Reset();
	CurrentGroupings.Reset();

	InternalCreateGroupings();

	if (CurrentGroupings.IsEmpty() && AvailableGroupings.Num() > 0)
	{
		CurrentGroupings.Add(AvailableGroupings[0]);
	}

	RebuildGroupingCrumbs();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::InternalCreateGroupings()
{
	AvailableGroupings.Add(MakeShared<FTreeNodeGroupingFlat>());
	//AvailableGroupings.Add(MakeShared<FTreeNodeGroupingByNameFirstLetter>());
	//AvailableGroupings.Add(MakeShared<FTreeNodeGroupingByType>());

	// By Unique Value
	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		const FTableColumn& Column = ColumnRef.Get();
		if (!Column.IsHierarchy())
		{
			switch (Column.GetDataType())
			{
				case ETableCellDataType::Bool:
					AvailableGroupings.Add(MakeShared<FTreeNodeGroupingByUniqueValueBool>(ColumnRef));
					break;

				case ETableCellDataType::Int64:
					AvailableGroupings.Add(MakeShared<FTreeNodeGroupingByUniqueValueInt64>(ColumnRef));
					break;

				case ETableCellDataType::Float:
					AvailableGroupings.Add(MakeShared<FTreeNodeGroupingByUniqueValueFloat>(ColumnRef));
					break;

				case ETableCellDataType::Double:
					AvailableGroupings.Add(MakeShared<FTreeNodeGroupingByUniqueValueDouble>(ColumnRef));
					break;

				case ETableCellDataType::CString:
					AvailableGroupings.Add(MakeShared<FTreeNodeGroupingByUniqueValueCString>(ColumnRef));
					break;

				default:
					AvailableGroupings.Add(MakeShared<FTreeNodeGroupingByUniqueValue>(ColumnRef));
			}
		}
	}

	// By Path Breakdown
	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		const FTableColumn& Column = ColumnRef.Get();
		if (!Column.IsHierarchy())
		{
			switch (Column.GetDataType())
			{
			case ETableCellDataType::Bool:
			case ETableCellDataType::Int64:
			case ETableCellDataType::Float:
			case ETableCellDataType::Double:
				break;

			case ETableCellDataType::CString:
				AvailableGroupings.Add(MakeShared<FTreeNodeGroupingByPathBreakdown>(ColumnRef));
				break;

			default:
				AvailableGroupings.Add(MakeShared<FTreeNodeGroupingByPathBreakdown>(ColumnRef));
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::PreChangeGroupings()
{
	// Show columns used in previous groupings.
	for (const FName& ColumnId : GroupingHiddenColumns)
	{
		ShowColumn(ColumnId);
	}
	GroupingHiddenColumns.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::PostChangeGroupings()
{
	constexpr bool bAdjustHierarchyColumnWidth = false;
	constexpr bool bAdjustHierarchyColumnName = false;

	constexpr float HierarchyMinWidth = 60.0f;
	constexpr float HierarchyIndentation = 10.0f;
	constexpr float DefaultHierarchyColumnWidth = 90.0f;

	float HierarchyColumnWidth = DefaultHierarchyColumnWidth;
	FString GroupingStr;

	float GroupingDepth = 0;
	for (TSharedPtr<FTreeNodeGrouping>& GroupingPtr : CurrentGroupings)
	{
		const FName& ColumnId = GroupingPtr->GetColumnId();
		if (ColumnId != NAME_None)
		{
			if (bAdjustHierarchyColumnWidth)
			{
				// Compute width for Hierarchy column based on column used in grouping and its indentation.
				const TIndirectArray<SHeaderRow::FColumn>& HeaderRowColumns = TreeViewHeaderRow->GetColumns();
				for (const SHeaderRow::FColumn& CurrentColumn : HeaderRowColumns)
				{
					if (CurrentColumn.ColumnId == ColumnId)
					{
						const float Width = HierarchyMinWidth + GroupingDepth + CurrentColumn.GetWidth();
						if (Width > HierarchyColumnWidth)
						{
							HierarchyColumnWidth = Width;
						}
						break;
					}
				}
			}

			// Hide columns used in groupings.
			FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
			if (Column.IsVisible())
			{
				GroupingHiddenColumns.Add(ColumnId);
				HideColumn(Column);
			}
		}

		if (bAdjustHierarchyColumnName)
		{
			// Compute name of the Hierarchy column.
			if (!GroupingStr.IsEmpty())
			{
				GroupingStr.Append(TEXT(" / "));
			}
			GroupingStr.Append(GroupingPtr->GetShortName().ToString());
		}

		GroupingDepth += HierarchyIndentation;
	}

	//////////////////////////////////////////////////

	if (TreeViewHeaderRow->GetColumns().Num() > 0)
	{
		if (bAdjustHierarchyColumnWidth)
		{
			// Set width for the Hierarchy column.
			SHeaderRow::FColumn& HierarchyColumn = const_cast<SHeaderRow::FColumn&>(TreeViewHeaderRow->GetColumns()[0]);
			HierarchyColumn.SetWidth(HierarchyColumnWidth);
		}

		if (bAdjustHierarchyColumnName)
		{
			// Set name for the Hierarchy column.
			SHeaderRow::FColumn& HierarchyColumn = const_cast<SHeaderRow::FColumn&>(TreeViewHeaderRow->GetColumns()[0]);
			FTableColumn& HierarchyTableColumn = *Table->FindColumnChecked(HierarchyColumn.ColumnId);
			if (!GroupingStr.IsEmpty())
			{
				const FText HierarchyColumnName = FText::Format(LOCTEXT("HierarchyShortNameFmt", "Hierarchy ({0})"), FText::FromString(GroupingStr));
				HierarchyTableColumn.SetShortName(HierarchyColumnName);
			}
			else
			{
				const FText HierarchyColumnName(LOCTEXT("HierarchyShortName", "Hierarchy"));
				HierarchyTableColumn.SetShortName(HierarchyColumnName);
			}
		}
	}

	//////////////////////////////////////////////////

	TreeViewHeaderRow->RefreshColumns();

	OnGroupingChanged();

	RebuildGroupingCrumbs();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::RebuildGroupingCrumbs()
{
	GroupingBreadcrumbTrail->ClearCrumbs();

	for (const TSharedPtr<FTreeNodeGrouping>& Grouping : CurrentGroupings)
	{
		GroupingBreadcrumbTrail->PushCrumb(Grouping->GetShortName(), Grouping);
	}

	GroupingBreadcrumbTrail->ScrollToEnd();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 STableTreeView::GetGroupingDepth(const TSharedPtr<FTreeNodeGrouping>& Grouping) const
{
	for (int32 GroupingDepth = CurrentGroupings.Num() - 1; GroupingDepth >= 0; --GroupingDepth)
	{
		if (Grouping == CurrentGroupings[GroupingDepth])
		{
			return GroupingDepth;
		}
	}
	return -1;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::OnGroupingCrumbClicked(const TSharedPtr<FTreeNodeGrouping>& CrumbGrouping)
{
	const int32 CrumbGroupingDepth = GetGroupingDepth(CrumbGrouping);
	if (CrumbGroupingDepth >= 0 && CrumbGroupingDepth < CurrentGroupings.Num() - 1)
	{
		PreChangeGroupings();

		CurrentGroupings.RemoveAt(CrumbGroupingDepth + 1, CurrentGroupings.Num() - CrumbGroupingDepth - 1);

		PostChangeGroupings();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STableTreeView::CreateGroupingMenuWidget(const FTreeNodeGrouping& Grouping) const
{
	return SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.Padding(FMargin(3.0f, 0.0f, 8.0f, 0.0f))
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(Grouping.GetIcon() ? Grouping.GetIcon() : FBaseTreeNode::GetDefaultIcon(true))
			.DesiredSizeOverride(FVector2D(12.0, 12.0))
			.ColorAndOpacity(Grouping.GetColor())
		]

		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 0.0f, 12.0f, 0.0f))
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(Grouping.GetTitleName())
		];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::AddGroupingMenuEntries(FMenuBuilder& MenuBuilder, EGroupingMenuType MenuType, const TSharedPtr<FTreeNodeGrouping> CrumbGrouping, TFunction<bool(const FTreeNodeGrouping&)> FilterFunc)
{
	for (const TSharedPtr<FTreeNodeGrouping>& Grouping : AvailableGroupings)
	{
		if (!FilterFunc(*Grouping))
		{
			continue;
		}

		FUIAction Action;
		if (MenuType == EGroupingMenuType::Change)
		{
			Action.ExecuteAction = FExecuteAction::CreateSP(this, &STableTreeView::GroupingCrumbMenu_Change_Execute, CrumbGrouping, Grouping);
			Action.CanExecuteAction = FCanExecuteAction::CreateSP(this, &STableTreeView::GroupingCrumbMenu_Change_CanExecute, CrumbGrouping, Grouping);
		}
		else // if (MenuType == EGroupingMenuType::Add)
		{
			Action.ExecuteAction = FExecuteAction::CreateSP(this, &STableTreeView::GroupingCrumbMenu_Add_Execute, Grouping, CrumbGrouping);
			Action.CanExecuteAction = FCanExecuteAction::CreateSP(this, &STableTreeView::GroupingCrumbMenu_Add_CanExecute, Grouping, CrumbGrouping);
		}

		MenuBuilder.AddMenuEntry
		(
			Action,
			CreateGroupingMenuWidget(*Grouping),
			NAME_None,
			Grouping->GetDescription(),
			EUserInterfaceActionType::Button
		);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::BuildGroupingSubMenu(FMenuBuilder& MenuBuilder, EGroupingMenuType MenuType, const TSharedPtr<FTreeNodeGrouping> CrumbGrouping)
{
	int32 NumUniqueValuesGroupings = 0;
	int32 NumPathBreakdownGroupings = 0;
	for (const TSharedPtr<FTreeNodeGrouping>& Grouping : AvailableGroupings)
	{
		if (Grouping->Is<FTreeNodeGroupingByUniqueValue>())
		{
			++NumUniqueValuesGroupings;
		}
		if (Grouping->Is<FTreeNodeGroupingByPathBreakdown>())
		{
			++NumPathBreakdownGroupings;
		}
	}
	const bool bSubMenuForUniqueValuesGroupings = (NumUniqueValuesGroupings > 3);
	const bool bSubMenuForPathBreakdownGroupings = (NumPathBreakdownGroupings > 3);

	MenuBuilder.BeginSection(MenuType == EGroupingMenuType::Change ? "ChangeGrouping" : "AddGrouping");

	if (bSubMenuForUniqueValuesGroupings || bSubMenuForPathBreakdownGroupings)
	{
		auto Filter = [bSubMenuForUniqueValuesGroupings, bSubMenuForPathBreakdownGroupings](const FTreeNodeGrouping& Grouping)
			{
				return (!(bSubMenuForUniqueValuesGroupings && Grouping.Is<FTreeNodeGroupingByUniqueValue>()) &&
						!(bSubMenuForPathBreakdownGroupings && Grouping.Is<FTreeNodeGroupingByPathBreakdown>()));
			};
		AddGroupingMenuEntries(MenuBuilder, MenuType, CrumbGrouping, Filter);

		if (bSubMenuForUniqueValuesGroupings)
		{
			TFunction<bool(const FTreeNodeGrouping&)> FilterByUniqueValue = [](const FTreeNodeGrouping& Grouping)
				{
					return Grouping.Is<FTreeNodeGroupingByUniqueValue>();
				};
			MenuBuilder.AddSubMenu
			(
				LOCTEXT("UniqueValuesGroupings", "Unique Values"),
				LOCTEXT("UniqueValuesGroupings_ToolTip", "Unique Values groupings"),
				FNewMenuDelegate::CreateSP(this, &STableTreeView::AddGroupingMenuEntries, MenuType, CrumbGrouping, FilterByUniqueValue),
				false,
				FSlateIcon()
			);
		}

		if (bSubMenuForPathBreakdownGroupings)
		{
			TFunction<bool(const FTreeNodeGrouping&)> FilterByPathBreakdown = [](const FTreeNodeGrouping& Grouping)
				{
					return Grouping.Is<FTreeNodeGroupingByPathBreakdown>();
				};
			MenuBuilder.AddSubMenu
			(
				LOCTEXT("PathBreakdownGroupings", "Path Breakdown"),
				LOCTEXT("PathBreakdownGroupings_ToolTip", "Path Breakdown groupings"),
				FNewMenuDelegate::CreateSP(this, &STableTreeView::AddGroupingMenuEntries, MenuType, CrumbGrouping, FilterByPathBreakdown),
				false,
				FSlateIcon()
			);
		}
	}
	else
	{
		auto Filter = [](const FTreeNodeGrouping& Grouping) { return true; };
		AddGroupingMenuEntries(MenuBuilder, MenuType, CrumbGrouping, Filter);
	}

	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STableTreeView::GetGroupingCrumbButtonContent(const TSharedPtr<FTreeNodeGrouping>& CrumbGrouping, const FTextBlockStyle* InTextStyle)
{
	return SNew(STextBlock)
		.Text(CrumbGrouping->GetShortName())
		.ColorAndOpacity(CrumbGrouping->GetColor());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STableTreeView::GetGroupingCrumbMenuContent(const TSharedPtr<FTreeNodeGrouping>& CrumbGrouping)
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, NULL);

	const int32 CrumbGroupingDepth = GetGroupingDepth(CrumbGrouping);

	MenuBuilder.BeginSection("InsertOrAdd");
	{
		const FText AddGroupingText = (CrumbGroupingDepth == CurrentGroupings.Num() - 1) ? // after last one
			LOCTEXT("GroupingMenu_Add_SubMenu", "Add Grouping...") :
			LOCTEXT("GroupingMenu_Insert_SubMenu", "Insert Grouping...");
		MenuBuilder.AddSubMenu
		(
			AddGroupingText,
			LOCTEXT("GroupingMenu_AddOrInsert_SubMenu_Desc", "Adds or inserts a new grouping."),
			FNewMenuDelegate::CreateSP(this, &STableTreeView::BuildGroupingSubMenu, EGroupingMenuType::Add, CrumbGrouping),
			false,
			FSlateIcon()
		);
	}
	MenuBuilder.EndSection();

	auto CanExecute = []()
	{
		return true;
	};

	if (CrumbGroupingDepth >= 0)
	{
		MenuBuilder.BeginSection("CrumbGrouping", CrumbGrouping->GetTitleName());
		{
			MenuBuilder.AddSubMenu
			(
				LOCTEXT("GroupingMenu_Change_SubMenu", "Change To..."),
				LOCTEXT("GroupingMenu_Change_SubMenu_Desc", "Changes the selected grouping."),
				FNewMenuDelegate::CreateSP(this, &STableTreeView::BuildGroupingSubMenu, EGroupingMenuType::Change, CrumbGrouping),
				false,
				FSlateIcon()
			);

			if (CrumbGroupingDepth > 0)
			{
				FUIAction Action_MoveLeft
				(
					FExecuteAction::CreateSP(this, &STableTreeView::GroupingCrumbMenu_MoveLeft_Execute, CrumbGrouping),
					FCanExecuteAction::CreateLambda(CanExecute)
				);
				MenuBuilder.AddMenuEntry
				(
					LOCTEXT("GroupingMenu_MoveLeft", "Move Left"),
					LOCTEXT("GroupingMenu_MoveLeft_Desc", "Moves the selected grouping to the left."),
					FSlateIcon(),
					Action_MoveLeft,
					NAME_None,
					EUserInterfaceActionType::Button
				);
			}

			if (CrumbGroupingDepth < CurrentGroupings.Num() - 1)
			{
				FUIAction Action_MoveRight
				(
					FExecuteAction::CreateSP(this, &STableTreeView::GroupingCrumbMenu_MoveRight_Execute, CrumbGrouping),
					FCanExecuteAction::CreateLambda(CanExecute)
				);
				MenuBuilder.AddMenuEntry
				(
					LOCTEXT("GroupingMenu_MoveRight", "Move Right"),
					LOCTEXT("GroupingMenu_MoveRight_Desc", "Moves the selected grouping to the right."),
					FSlateIcon(),
					Action_MoveRight,
					NAME_None,
					EUserInterfaceActionType::Button
				);
			}

			if (CurrentGroupings.Num() > 1)
			{
				FUIAction Action_Remove
				(
					FExecuteAction::CreateSP(this, &STableTreeView::GroupingCrumbMenu_Remove_Execute, CrumbGrouping),
					FCanExecuteAction::CreateLambda(CanExecute)
				);
				MenuBuilder.AddMenuEntry
				(
					LOCTEXT("GroupingMenu_Remove", "Remove"),
					LOCTEXT("GroupingMenu_Remove_Desc", "Removes the selected grouping."),
					FSlateIcon(),
					Action_Remove,
					NAME_None,
					EUserInterfaceActionType::Button
				);
			}
		}
		MenuBuilder.EndSection();
	}

	if (CurrentGroupings.Num() > 1 || CurrentGroupings[0] != AvailableGroupings[0])
	{
		MenuBuilder.BeginSection("ResetGroupings");
		{
			FUIAction Action_Reset
			(
				FExecuteAction::CreateSP(this, &STableTreeView::GroupingCrumbMenu_Reset_Execute),
				FCanExecuteAction::CreateLambda(CanExecute)
			);
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("GroupingMenu_Reset", "Reset"),
				LOCTEXT("GroupingMenu_Reset_Desc", "Resets groupings to default."),
				FSlateIcon(),
				Action_Reset,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
		MenuBuilder.EndSection();
	}

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::GroupingCrumbMenu_Reset_Execute()
{
	PreChangeGroupings();

	CurrentGroupings.Reset();
	CurrentGroupings.Add(AvailableGroupings[0]);

	PostChangeGroupings();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::GroupingCrumbMenu_Remove_Execute(const TSharedPtr<FTreeNodeGrouping> Grouping)
{
	const int32 GroupingDepth = GetGroupingDepth(Grouping);
	if (GroupingDepth >= 0)
	{
		PreChangeGroupings();

		CurrentGroupings.RemoveAt(GroupingDepth);

		PostChangeGroupings();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::GroupingCrumbMenu_MoveLeft_Execute(const TSharedPtr<FTreeNodeGrouping> Grouping)
{
	const int32 GroupingDepth = GetGroupingDepth(Grouping);
	if (GroupingDepth > 0)
	{
		PreChangeGroupings();

		CurrentGroupings[GroupingDepth] = CurrentGroupings[GroupingDepth - 1];
		CurrentGroupings[GroupingDepth - 1] = Grouping;

		PostChangeGroupings();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::GroupingCrumbMenu_MoveRight_Execute(const TSharedPtr<FTreeNodeGrouping> Grouping)
{
	const int32 GroupingDepth = GetGroupingDepth(Grouping);
	if (GroupingDepth < CurrentGroupings.Num() - 1)
	{
		PreChangeGroupings();

		CurrentGroupings[GroupingDepth] = CurrentGroupings[GroupingDepth + 1];
		CurrentGroupings[GroupingDepth + 1] = Grouping;

		PostChangeGroupings();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::GroupingCrumbMenu_Change_Execute(const TSharedPtr<FTreeNodeGrouping> OldGrouping, const TSharedPtr<FTreeNodeGrouping> NewGrouping)
{
	const int32 OldGroupingDepth = GetGroupingDepth(OldGrouping);
	if (OldGroupingDepth >= 0)
	{
		PreChangeGroupings();

		const int32 NewGroupingDepth = GetGroupingDepth(NewGrouping);

		if (NewGroupingDepth >= 0 && NewGroupingDepth != OldGroupingDepth) // NewGrouping already exists
		{
			CurrentGroupings.RemoveAt(NewGroupingDepth);

			if (NewGroupingDepth < OldGroupingDepth)
			{
				CurrentGroupings[OldGroupingDepth - 1] = NewGrouping;
			}
			else
			{
				CurrentGroupings[OldGroupingDepth] = NewGrouping;
			}
		}
		else
		{
			CurrentGroupings[OldGroupingDepth] = NewGrouping;
		}

		PostChangeGroupings();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::GroupingCrumbMenu_Change_CanExecute(const TSharedPtr<FTreeNodeGrouping> OldGrouping, const TSharedPtr<FTreeNodeGrouping> NewGrouping) const
{
	return NewGrouping != OldGrouping;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::GroupingCrumbMenu_Add_Execute(const TSharedPtr<FTreeNodeGrouping> NewGrouping, const TSharedPtr<FTreeNodeGrouping> AfterGrouping)
{
	PreChangeGroupings();

	if (AfterGrouping.IsValid())
	{
		const int32 AfterGroupingDepth = GetGroupingDepth(AfterGrouping);
		ensure(AfterGroupingDepth >= 0);

		const int32 GroupingDepth = GetGroupingDepth(NewGrouping);

		if (GroupingDepth >= 0) // Grouping already exists
		{
			CurrentGroupings.RemoveAt(GroupingDepth);

			if (GroupingDepth <= AfterGroupingDepth)
			{
				CurrentGroupings.Insert(NewGrouping, AfterGroupingDepth);
			}
			else
			{
				CurrentGroupings.Insert(NewGrouping, AfterGroupingDepth + 1);
			}
		}
		else
		{
			CurrentGroupings.Insert(NewGrouping, AfterGroupingDepth + 1);
		}
	}
	else
	{
		CurrentGroupings.Remove(NewGrouping);
		CurrentGroupings.Add(NewGrouping);
	}

	PostChangeGroupings();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::GroupingCrumbMenu_Add_CanExecute(const TSharedPtr<FTreeNodeGrouping> NewGrouping, const TSharedPtr<FTreeNodeGrouping> AfterGrouping) const
{
	if (AfterGrouping.IsValid())
	{
		const int32 AfterGroupingDepth = GetGroupingDepth(AfterGrouping);
		ensure(AfterGroupingDepth >= 0);

		const int32 GroupingDepth = GetGroupingDepth(NewGrouping);

		return GroupingDepth < AfterGroupingDepth || GroupingDepth > AfterGroupingDepth + 1;
	}
	else
	{
		return NewGrouping != CurrentGroupings.Last();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Aggregation
////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::UpdateCStringSameValueAggregationSingleNode(const FTableColumn& InColumn, FTableTreeNode& InOutGroupNode)
{
	check(InOutGroupNode.IsGroup());

	const TCHAR* AggregatedValue = nullptr;

	// Find the first child node.
	if (InOutGroupNode.GetChildrenCount() > 0)
	{
		FBaseTreeNodePtr NodePtr = InOutGroupNode.GetChildren()[0];
		const TOptional<FTableCellValue> NodeValue = InColumn.GetValue(*NodePtr);
		if (NodeValue.IsSet() &&
			NodeValue.GetValue().DataType == ETableCellDataType::CString)
		{
			AggregatedValue = NodeValue.GetValue().CString;
		}
	}

	if (AggregatedValue != nullptr)
	{
		// Check if all other children have the same value as the first node.
		for (FBaseTreeNodePtr NodePtr : InOutGroupNode.GetChildren())
		{
			const TOptional<FTableCellValue> NodeValue = InColumn.GetValue(*NodePtr);
			if (NodeValue.IsSet() &&
				NodeValue.GetValue().DataType == ETableCellDataType::CString &&
				AggregatedValue == NodeValue.GetValue().CString)
			{
				continue;
			}

			AggregatedValue = nullptr;
			break;
		}
	}

	if (AggregatedValue != nullptr)
	{
		InOutGroupNode.SetAggregatedValue(InColumn.GetId(), FTableCellValue(AggregatedValue));
	}
	else
	{
		InOutGroupNode.ResetAggregatedValue(InColumn.GetId());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::UpdateCStringSameValueAggregationRec(const FTableColumn& InColumn, FTableTreeNode& InOutGroupNode)
{
	check(InOutGroupNode.IsGroup());

	// Update child group nodes first.
	for (FBaseTreeNodePtr NodePtr : InOutGroupNode.GetChildren())
	{
		if (NodePtr->IsGroup())
		{
			FTableTreeNode& TableNode = *(FTableTreeNode*)NodePtr.Get();
			STableTreeView::UpdateCStringSameValueAggregationRec(InColumn, TableNode);
		}
	}

	STableTreeView::UpdateCStringSameValueAggregationSingleNode(InColumn, InOutGroupNode);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename T, bool bSetInitialValue, bool bIsRercursive>
void STableTreeView::UpdateAggregation(const FTableColumn& InColumn, FTableTreeNode& InOutGroupNode, const T InitialAggregatedValue, TFunctionRef<T(T, const FTableCellValue&)> ValueGetterFunc)
{
	T AggregatedValue = InitialAggregatedValue;

	for (FBaseTreeNodePtr NodePtr : InOutGroupNode.GetChildren())
	{
		if (NodePtr->IsGroup())
		{
			FTableTreeNode& TableNode = *(FTableTreeNode*)NodePtr.Get();
			if (bIsRercursive)
			{
				TableNode.ResetAggregatedValue(InColumn.GetId());
				STableTreeView::UpdateAggregation<T, bSetInitialValue, true>(InColumn, TableNode, InitialAggregatedValue, ValueGetterFunc);
			}
		}
		const TOptional<FTableCellValue> NodeValue = InColumn.GetValue(*NodePtr);
		if (NodeValue.IsSet())
		{
			AggregatedValue = ValueGetterFunc(AggregatedValue, NodeValue.GetValue());
		}
	}

	if (bSetInitialValue || AggregatedValue != InitialAggregatedValue)
	{
		InOutGroupNode.SetAggregatedValue(InColumn.GetId(), FTableCellValue(AggregatedValue));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

template<bool bIsRecursive>
void STableTreeView::UpdateAggregatedValues(TSharedPtr<FTable> InTable, FTableTreeNode& InOutGroupNode)
{
	FStopwatch Stopwatch;
	Stopwatch.Start();

	check(InOutGroupNode.IsGroup());

	for (const TSharedRef<FTableColumn>& ColumnRef : InTable->GetColumns())
	{
		FTableColumn& Column = ColumnRef.Get();
		switch (Column.GetAggregation())
		{
			case ETableColumnAggregation::Sum:
				if (Column.GetDataType() == ETableCellDataType::Float || Column.GetDataType() == ETableCellDataType::Double)
				{
					STableTreeView::UpdateAggregation<double, true, bIsRecursive>(Column, InOutGroupNode, 0.0,
						[](double InValue, const FTableCellValue& InTableCellValue)
						{
							return InValue + InTableCellValue.AsDouble();
						});
				}
				else
				{
					STableTreeView::UpdateAggregation<int64, true, bIsRecursive>(Column, InOutGroupNode, (int64)0,
						[](int64 InValue, const FTableCellValue& InTableCellValue)
						{
							return InValue + InTableCellValue.AsInt64();
						});
				}
				break;

			case ETableColumnAggregation::Min:
				if (Column.GetDataType() == ETableCellDataType::Float || Column.GetDataType() == ETableCellDataType::Double)
				{
					STableTreeView::UpdateAggregation<double, false, bIsRecursive>(Column, InOutGroupNode, std::numeric_limits<double>::max(),
						[](double InValue, const FTableCellValue& InTableCellValue)
						{
							return FMath::Min(InValue, InTableCellValue.AsDouble());
						});
				}
				else
				{
					STableTreeView::UpdateAggregation<int64, false, bIsRecursive>(Column, InOutGroupNode, std::numeric_limits<int64>::max(),
						[](int64 InValue, const FTableCellValue& InTableCellValue)
						{
							return FMath::Min(InValue, InTableCellValue.AsInt64());
						});
				}
				break;

			case ETableColumnAggregation::Max:
				if (Column.GetDataType() == ETableCellDataType::Float || Column.GetDataType() == ETableCellDataType::Double)
				{
					STableTreeView::UpdateAggregation<double, false, bIsRecursive>(Column, InOutGroupNode, std::numeric_limits<double>::lowest(),
						[](double InValue, const FTableCellValue& InTableCellValue)
						{
							return FMath::Max(InValue, InTableCellValue.AsDouble());
						});
				}
				else
				{
					STableTreeView::UpdateAggregation<int64, false, bIsRecursive>(Column, InOutGroupNode, std::numeric_limits<int64>::min(),
						[](int64 InValue, const FTableCellValue& InTableCellValue)
						{
							return FMath::Max(InValue, InTableCellValue.AsInt64());
						});
				}
				break;

			case ETableColumnAggregation::SameValue:
				if (Column.GetDataType() == ETableCellDataType::CString)
				{
					if (bIsRecursive)
					{
						STableTreeView::UpdateCStringSameValueAggregationRec(Column, InOutGroupNode);
					}
					else
					{
						STableTreeView::UpdateCStringSameValueAggregationSingleNode(Column, InOutGroupNode);
					}
				}
				break;
		}
	}

	Stopwatch.Stop();
	const double AggregationTime = Stopwatch.GetAccumulatedTime();
	if (AggregationTime > 0.1)
	{
		UE_LOGF(LogInsights, Log, "[Tree - %ls] Aggregation completed in %.3fs.", *InTable->GetDisplayName().ToString(), AggregationTime);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::UpdateAggregatedValuesSingleNode(FTableTreeNode& InOutGroupNode)
{
	STableTreeView::UpdateAggregatedValues<false>(Table, InOutGroupNode);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::UpdateAggregatedValuesRec(FTableTreeNode& InOutGroupNode)
{
	if (AsyncOperationProgress.ShouldCancelAsyncOp())
	{
		return;
	}

	STableTreeView::UpdateAggregatedValues<true>(Table, InOutGroupNode);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Sorting
////////////////////////////////////////////////////////////////////////////////////////////////////

const EColumnSortMode::Type STableTreeView::GetDefaultColumnSortMode()
{
	return EColumnSortMode::Descending;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FName STableTreeView::GetDefaultColumnBeingSorted()
{
	return NAME_None;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::CreateSortings()
{
	AvailableSorters.Reset();
	CurrentSorter = nullptr;

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		const FTableColumn& Column = ColumnRef.Get();
		if (Column.CanBeSorted())
		{
			TSharedPtr<ITableCellValueSorter> SorterPtr = Column.GetValueSorter();
			if (ensure(SorterPtr.IsValid()))
			{
				AvailableSorters.Add(SorterPtr);
			}
		}
	}

	UpdateCurrentSortingByColumn();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::UpdateCurrentSortingByColumn()
{
	TSharedPtr<FTableColumn> ColumnPtr = Table->FindColumn(ColumnBeingSorted);
	CurrentSorter = ColumnPtr.IsValid() ? ColumnPtr->GetValueSorter() : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::OnSortingChanged()
{
	if (!bRunInAsyncMode)
	{
		ApplySorting();
		ApplyHierarchyFiltering();
	}
	else
	{
		AddPendingAsyncOperation(EAsyncOperationType::Sorting);
		if (!bIsUpdateRunning)
		{
			ScheduleSortingAsyncOperation();
		}
		else
		{
			CancelCurrentAsyncOp();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::ScheduleSortingAsyncOperationIfNeeded()
{
	bool bIsSortingNeeded =
		HasPendingAsyncOperation(EAsyncOperationType::Sorting) ||
		HasInProgressAsyncOperation(EAsyncOperationType::Sorting);

	if (!bIsSortingNeeded &&
		CurrentSorter.Get() != CurrentAsyncOpSorter)
	{
		bIsSortingNeeded = true;
	}

	if (!bIsSortingNeeded &&
		CurrentSorter.IsValid() &&
		ColumnSortMode != CurrentAsyncOpColumnSortMode)
	{
		bIsSortingNeeded = true;
	}

	if (bIsSortingNeeded)
	{
		ScheduleSortingAsyncOperation();
		return true;
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::ScheduleSortingAsyncOperation()
{
	OnPreAsyncUpdate();

	ClearPendingAsyncOperation(EAsyncOperationType::Sorting);

	FGraphEventRef CompletedEvent = StartSortingTask();
	InProgressAsyncOperationEvent = StartHierarchyFilteringTask(CompletedEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FGraphEventRef STableTreeView::StartSortingTask(FGraphEventRef Prerequisite)
{
	if (!CurrentSorter.IsValid())
	{
		return Prerequisite;
	}

	AddInProgressAsyncOperation(EAsyncOperationType::Sorting);

	CurrentAsyncOpSorter = CurrentSorter.Get();
	CurrentAsyncOpColumnSortMode = ColumnSortMode;

	FGraphEventArray Prerequisites;
	if (Prerequisite.IsValid())
	{
		Prerequisites.Add(Prerequisite);
	}
	else
	{
		Prerequisites.Add(DispatchEvent);
	}

	return TGraphTask<FTableTreeViewSortingAsyncTask>::CreateTask(&Prerequisites).ConstructAndDispatchWhenReady(this, CurrentAsyncOpSorter, CurrentAsyncOpColumnSortMode);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::ApplySorting()
{
	if (CurrentSorter.IsValid())
	{
		SortTreeNodes(CurrentSorter.Get(), ColumnSortMode);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

// Note: This function is called from a task thread!
void STableTreeView::SortTreeNodes(ITableCellValueSorter* InSorter, EColumnSortMode::Type InColumnSortMode)
{
	FStopwatch Stopwatch;
	Stopwatch.Start();
#if !PLATFORM_EXCEPTIONS_DISABLED
	try
#endif
	{
		InSorter->SetAsyncOperationProgress(&AsyncOperationProgress);
		SortTreeNodesRec(*Root, *InSorter, InColumnSortMode);
	}
#if !PLATFORM_EXCEPTIONS_DISABLED
	catch (const char* Exception)
	{
		if (FCStringAnsi::Strcmp(Exception, "Cancelling sort"))
		{
			throw Exception;
		}
	}
#endif

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	if (TotalTime > 0.1)
	{
		UE_LOGF(LogInsights, Log, "[Tree - %ls] Sorting completed in %.3fs.", *Table->GetDisplayName().ToString(), TotalTime);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::SortTreeNodesRec(FTableTreeNode& GroupNode, const ITableCellValueSorter& Sorter, EColumnSortMode::Type InColumnSortMode)
{
	const ESortMode SortMode = (InColumnSortMode == EColumnSortMode::Type::Descending) ? ESortMode::Descending : ESortMode::Ascending;
	GroupNode.SortChildren(Sorter, SortMode);

	for (FBaseTreeNodePtr ChildPtr : GroupNode.GetChildren())
	{
		if (AsyncOperationProgress.ShouldCancelAsyncOp())
		{
			break;
		}
		if (ChildPtr->IsGroup())
		{
			SortTreeNodesRec(*StaticCastSharedPtr<FTableTreeNode>(ChildPtr), Sorter, InColumnSortMode);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EColumnSortMode::Type STableTreeView::GetSortModeForColumn(const FName ColumnId) const
{
	if (ColumnBeingSorted != ColumnId)
	{
		return EColumnSortMode::None;
	}

	return ColumnSortMode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::SetSortModeForColumn(const FName& ColumnId, const EColumnSortMode::Type SortMode)
{
	ColumnBeingSorted = ColumnId;
	ColumnSortMode = SortMode;
	UpdateCurrentSortingByColumn();

	OnSortingChanged();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::OnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type SortMode)
{
	SetSortModeForColumn(ColumnId, SortMode);
	TreeView_Refresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// SortMode action (HeaderMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::HeaderMenu_SortMode_IsChecked(const FName ColumnId, const EColumnSortMode::Type InSortMode)
{
	return ColumnBeingSorted == ColumnId && ColumnSortMode == InSortMode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::HeaderMenu_SortMode_CanExecute(const FName ColumnId, const EColumnSortMode::Type InSortMode) const
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return Column.CanBeSorted();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::HeaderMenu_SortMode_Execute(const FName ColumnId, const EColumnSortMode::Type InSortMode)
{
	SetSortModeForColumn(ColumnId, InSortMode);
	TreeView_Refresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// SortMode action (ContextMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::ContextMenu_SortMode_IsChecked(const EColumnSortMode::Type InSortMode)
{
	return ColumnSortMode == InSortMode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::ContextMenu_SortMode_CanExecute(const EColumnSortMode::Type InSortMode) const
{
	return true; //ColumnSortMode != InSortMode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::ContextMenu_SortMode_Execute(const EColumnSortMode::Type InSortMode)
{
	SetSortModeForColumn(ColumnBeingSorted, InSortMode);
	TreeView_Refresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// SortByColumn action (ContextMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::ContextMenu_SortByColumn_IsChecked(const FName ColumnId)
{
	return ColumnId == ColumnBeingSorted;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::ContextMenu_SortByColumn_CanExecute(const FName ColumnId) const
{
	return true; //ColumnId != ColumnBeingSorted;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::ContextMenu_SortByColumn_Execute(const FName ColumnId)
{
	SetSortModeForColumn(ColumnId, EColumnSortMode::Descending);
	TreeView_Refresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// ShowColumn action
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::CanShowColumn(const FName ColumnId) const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::ShowColumn(const FName ColumnId)
{
	FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	ShowColumn(Column);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::ShowColumn(FTableColumn& Column)
{
	if (!Column.IsVisible())
	{
		Column.Show();

		SHeaderRow::FColumn::FArguments ColumnArgs;
		ColumnArgs
			.ColumnId(Column.GetId())
			.DefaultLabel(Column.GetShortName())
			.ToolTip(STableTreeViewTooltip::GetColumnTooltip(Column))
			.HAlignHeader(Column.GetHorizontalAlignment())
			.VAlignHeader(VAlign_Center)
			.HAlignCell(HAlign_Fill)
			.VAlignCell(VAlign_Fill)
			.InitialSortMode(Column.GetInitialSortMode())
			.SortMode(this, &STableTreeView::GetSortModeForColumn, Column.GetId())
			.FillWidth(Column.GetInitialWidth())
			//.FixedWidth(Column.IsFixedWidth() ? Column.GetInitialWidth() : TOptional<float>())
			.HeaderContent()
			[
				SNew(SBox)
				.HeightOverride(24.0f)
				.Padding(FMargin(0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(this, &STableTreeView::GetColumnHeaderText, Column.GetId())
				]
			]
			.MenuContent()
			[
				TreeViewHeaderRow_GenerateColumnMenu(Column)
			];

		if (Column.CanBeSorted())
		{
			ColumnArgs.OnSort(this, &STableTreeView::OnSortModeChanged);
		}

		int32 ColumnIndex = 0;
		const int32 NewColumnPosition = Table->GetColumnPositionIndex(Column.GetId());
		const int32 NumColumns = TreeViewHeaderRow->GetColumns().Num();
		for (; ColumnIndex < NumColumns; ColumnIndex++)
		{
			const SHeaderRow::FColumn& CurrentColumn = TreeViewHeaderRow->GetColumns()[ColumnIndex];
			const int32 CurrentColumnPosition = Table->GetColumnPositionIndex(CurrentColumn.ColumnId);
			if (NewColumnPosition < CurrentColumnPosition)
			{
				break;
			}
		}

		TreeViewHeaderRow->InsertColumn(ColumnArgs, ColumnIndex);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// HideColumn action
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::CanHideColumn(const FName ColumnId) const
{
	if (bIsUpdateRunning)
	{
		return false;
	}

	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return Column.CanBeHidden();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::HideColumn(const FName ColumnId)
{
	FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	HideColumn(Column);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::HideColumn(FTableColumn& Column)
{
	if (Column.IsVisible())
	{
		Column.Hide();
		TreeViewHeaderRow->RemoveColumn(Column.GetId());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// ToggleColumn action
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::IsColumnVisible(const FName ColumnId)
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return Column.IsVisible();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::CanToggleColumnVisibility(const FName ColumnId) const
{
	if (bIsUpdateRunning)
	{
		return false;
	}

	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return !Column.IsVisible() || Column.CanBeHidden();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::ToggleColumnVisibility(const FName ColumnId)
{
	FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	if (Column.IsVisible())
	{
		HideColumn(Column);
	}
	else
	{
		ShowColumn(Column);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// "Show All Columns" action (ContextMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::ContextMenu_ShowAllColumns_CanExecute() const
{
	if (bIsUpdateRunning)
	{
		return false;
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::ContextMenu_ShowAllColumns_Execute()
{
	ColumnBeingSorted = GetDefaultColumnBeingSorted();
	ColumnSortMode = GetDefaultColumnSortMode();
	UpdateCurrentSortingByColumn();

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		FTableColumn& Column = ColumnRef.Get();
		if (!Column.IsVisible())
		{
			ShowColumn(Column);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// ResetColumns action (ContextMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::ContextMenu_ResetColumns_CanExecute() const
{
	if (bIsUpdateRunning)
	{
		return false;
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::ContextMenu_ResetColumns_Execute()
{
	ColumnBeingSorted = GetDefaultColumnBeingSorted();
	ColumnSortMode = GetDefaultColumnSortMode();
	UpdateCurrentSortingByColumn();

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		FTableColumn& Column = ColumnRef.Get();
		if (Column.ShouldBeVisible() && !Column.IsVisible())
		{
			ShowColumn(Column);
		}
		else if (!Column.ShouldBeVisible() && Column.IsVisible())
		{
			HideColumn(Column);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::ContextMenu_HideAllColumns_CanExecute() const
{
	if (bIsUpdateRunning)
	{
		return false;
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::ContextMenu_HideAllColumns_Execute()
{
	ColumnBeingSorted = GetDefaultColumnBeingSorted();
	ColumnSortMode = GetDefaultColumnSortMode();
	UpdateCurrentSortingByColumn();

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		FTableColumn& Column = ColumnRef.Get();
		if (Column.IsVisible() && Column.CanBeHidden())
		{
			HideColumn(Column);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::Reset()
{
	RebuildTree(true);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::RebuildTree(bool bResync)
{
	unimplemented();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTableTreeNodePtr STableTreeView::GetNodeByTableRowIndex(int32 RowIndex) const
{
	return (RowIndex >= 0 && RowIndex < TableRowNodes.Num()) ? TableRowNodes[RowIndex] : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::SelectNodeByTableRowIndex(int32 RowIndex)
{
	FTableTreeNodePtr NodePtr = GetNodeByTableRowIndex(RowIndex);
	if (NodePtr.IsValid())
	{
		// Expand all parent nodes.
		FBaseTreeNodePtr ParentNode = NodePtr->GetParent();
		while (ParentNode.IsValid())
		{
			TreeView->SetItemExpansion(StaticCastSharedPtr<FTableTreeNode>(ParentNode), true);
			ParentNode = ParentNode->GetParent();

		}

		TreeView->SetSelection(NodePtr);
		TreeView->RequestScrollIntoView(NodePtr);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::OnPreAsyncUpdate()
{
	check(!bIsUpdateRunning);

	ClearInProgressAsyncOperations();

	AsyncUpdateStopwatch.Restart();
	bIsUpdateRunning = true;

	TreeView->SetTreeItemsSource(&DummyGroupNodes);
	TreeView->RequestTreeRefresh();

	DispatchEvent = FGraphEvent::CreateGraphEvent();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::OnPostAsyncUpdate()
{
	check(bIsUpdateRunning);

	bIsUpdateRunning = false;

	if (!AsyncOperationProgress.ShouldCancelAsyncOp())
	{
		TreeView->SetTreeItemsSource(&FilteredGroupNodes);

		// Expand each group node on the first few depths (if it doesn't have too many children).
		SetExpandValueForChildGroups(Root.Get(), MaxNodesToAutoExpand, MaxDepthToAutoExpand, true);

		// Update node expansion for the tree view.
		TreeView->ClearExpandedItems();
		Root->SetExpansion(true);
		ApplyNodeExpansion(Root);

		TreeView->RequestTreeRefresh();

		ClearInProgressAsyncOperations();
	}

	AsyncOperationProgress.Reset();
	AsyncUpdateStopwatch.Stop();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::CountNumNodesPerDepthRec(FBaseTreeNode* InRoot, TArray<int32>& InOutNumNodesPerDepth, int32 InDepth, int32 InMaxDepth, int32 InMaxNodes) const
{
	InOutNumNodesPerDepth[InDepth] += InRoot->GetChildrenCount();

	if (InDepth < InMaxDepth && InOutNumNodesPerDepth[InDepth] < InMaxNodes)
	{
		for (const FBaseTreeNodePtr& Node : InRoot->GetChildren())
		{
			if (Node->IsGroup())
			{
				CountNumNodesPerDepthRec(Node.Get(), InOutNumNodesPerDepth, InDepth + 1, InMaxDepth, InMaxNodes);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::ApplyNodeExpansion(const FTableTreeNodePtr& InNode)
{
	if (InNode->IsGroup())
	{
		const bool bIsNodeExpanded = InNode->IsExpanded();

		TreeView->SetItemExpansion(InNode, bIsNodeExpanded);

		if (bIsNodeExpanded)
		{
			for (const FBaseTreeNodePtr& ChildNode : InNode->GetChildren())
			{
				ApplyNodeExpansion(StaticCastSharedPtr<FTableTreeNode>(ChildNode));
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::SetExpandValueForChildGroups(FBaseTreeNode* InRoot, int32 InMaxNodesToExpand, int32 InMaxDepthToExpand, bool InValue)
{
	if (!InRoot)
	{
		return;
	}

	TArray<int32> NumNodesPerDepth;
	NumNodesPerDepth.AddDefaulted(InMaxDepthToExpand + 1);
	CountNumNodesPerDepthRec(InRoot, NumNodesPerDepth, 0, InMaxDepthToExpand, InMaxNodesToExpand);

	int32 MaxDepth = 0;
	for (int32 Depth = 0; Depth <= InMaxDepthToExpand; ++Depth)
	{
		if (Depth > 0)
		{
			NumNodesPerDepth[Depth] += NumNodesPerDepth[Depth - 1];
		}
		if (NumNodesPerDepth[Depth] > InMaxNodesToExpand)
		{
			break;
		}
		MaxDepth = Depth;
	}

	if (MaxDepth > 0)
	{
		SetExpandValueForChildGroupsRec(InRoot, 1, MaxDepth, InValue);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::SetExpandValueForChildGroupsRec(FBaseTreeNode* InRoot, int32 InDepth, int32 InMaxDepth, bool InValue)
{
	for (const FBaseTreeNodePtr& Node : InRoot->GetChildren())
	{
		if (Node->IsGroup())
		{
			Node->SetExpansion(InValue);

			if (InDepth < InMaxDepth)
			{
				SetExpandValueForChildGroupsRec(Node.Get(), InDepth + 1, InMaxDepth, InValue);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::OnClose()
{
	if (bIsCloseScheduled)
	{
		return;
	}

	bIsCloseScheduled = true;

	if (bIsUpdateRunning)
	{
		CancelCurrentAsyncOp();
		check(InProgressAsyncOperationEvent.IsValid());
		InProgressAsyncOperationEvent->Wait(ENamedThreads::GameThread);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STableTreeView::GetCurrentOperationName() const
{
	return LOCTEXT("CurrentOperationName", "Updating Tree");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

double STableTreeView::GetAllOperationsDuration()
{
	AsyncUpdateStopwatch.Update();
	return AsyncUpdateStopwatch.GetAccumulatedTime();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::StartPendingAsyncOperations()
{
	if (ScheduleNodeFilteringAsyncOperationIfNeeded())
	{
		return;
	}

	if (ScheduleGroupingAsyncOperationIfNeeded())
	{
		return;
	}

	if (ScheduleSortingAsyncOperationIfNeeded())
	{
		return;
	}

	if (ScheduleHierarchyFilteringAsyncOperationIfNeeded())
	{
		return;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::CancelCurrentAsyncOp()
{
	if (bIsUpdateRunning)
	{
		if (DispatchEvent.IsValid() && !DispatchEvent->IsComplete())
		{
			DispatchEvent->DispatchSubsequents();
		}

		AsyncOperationProgress.CancelCurrentAsyncOp();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STableTreeView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	return CommandList->ProcessCommandBindings(InKeyEvent) == true ? FReply::Handled() : FReply::Unhandled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::ContextMenu_CopySelectedToClipboard_CanExecute() const
{
	return TreeView->GetNumItemsSelected() > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::ContextMenu_CopySelectedToClipboard_Execute()
{
	if (!Table->IsValid())
	{
		return;
	}

	TArray<FBaseTreeNodePtr> SelectedNodes;
	for (FTableTreeNodePtr TimerPtr : TreeView->GetSelectedItems())
	{
		SelectedNodes.Add(TimerPtr);
	}

	if (SelectedNodes.Num() == 0)
	{
		return;
	}

	FString ClipboardText;

	if (CurrentSorter.IsValid())
	{
		CurrentSorter->Sort(SelectedNodes, ColumnSortMode == EColumnSortMode::Ascending ? ESortMode::Ascending : ESortMode::Descending);
	}

	Table->GetVisibleColumnsData(SelectedNodes, GetLogListingName(), TEXT('\t'), true, ClipboardText);

	if (ClipboardText.Len() > 0)
	{
		FPlatformApplicationMisc::ClipboardCopy(*ClipboardText);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::ContextMenu_CopyColumnToClipboard_CanExecute() const
{
	const TSharedPtr<FTableColumn> HoveredColumnPtr = Table->FindColumn(HoveredColumnId);

	if (HoveredColumnPtr.IsValid() && TreeView->GetNumItemsSelected() == 1)
	{
		return true;
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::ContextMenu_CopyColumnToClipboard_Execute()
{
	if (TreeView->GetNumItemsSelected() > 0)
	{
		FTableTreeNodePtr SelectedNode = TreeView->GetSelectedItems()[0];
		const TSharedPtr<FTableColumn> HoveredColumnPtr = Table->FindColumn(HoveredColumnId);
		if (HoveredColumnPtr.IsValid())
		{
			FString Text = HoveredColumnPtr->CopyValue(*SelectedNode).ToString();
			FPlatformApplicationMisc::ClipboardCopy(*Text);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::ContextMenu_CopyColumnTooltipToClipboard_CanExecute() const
{
	const TSharedPtr<FTableColumn> HoveredColumnPtr = Table->FindColumn(HoveredColumnId);

	if (HoveredColumnPtr.IsValid() && TreeView->GetNumItemsSelected() == 1)
	{
		return true;
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::ContextMenu_CopyColumnTooltipToClipboard_Execute()
{
	if (TreeView->GetNumItemsSelected() > 0)
	{
		FTableTreeNodePtr SelectedNode = TreeView->GetSelectedItems()[0];
		const TSharedPtr<FTableColumn> HoveredColumnPtr = Table->FindColumn(HoveredColumnId);
		if (HoveredColumnPtr.IsValid())
		{
			FString Text = HoveredColumnPtr->CopyTooltip(*SelectedNode).ToString();
			FPlatformApplicationMisc::ClipboardCopy(*Text);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::ContextMenu_ExpandSubtree_CanExecute() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::ContextMenu_ExpandSubtree_Execute()
{
	TArray<FTableTreeNodePtr> SelectedNodes;
	if (!TreeView->GetSelectedItems(SelectedNodes))
	{
		return;
	}

	for (FTableTreeNodePtr& Node : SelectedNodes)
	{
		if (Node->IsGroup())
		{
			Node->SetExpansion(true);
			SetExpandValueForChildGroups((FBaseTreeNode*)Node.Get(), MaxNodesToExpand, MaxDepthToExpand, true);
		}
	}

	TreeView->ClearExpandedItems();
	ApplyNodeExpansion(Root);
	TreeView->RequestTreeRefresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::ContextMenu_CollapseSubtree_CanExecute() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::ContextMenu_CollapseSubtree_Execute()
{
	TArray<FTableTreeNodePtr> SelectedNodes;
	if (!TreeView->GetSelectedItems(SelectedNodes))
	{
		return;
	}

	for (FTableTreeNodePtr& Node : SelectedNodes)
	{
		if (Node->IsGroup())
		{
			Node->SetExpansion(false);
			SetExpandValueForChildGroups((FBaseTreeNode*)Node.Get(), MaxNodesToExpand, MaxDepthToExpand, false);
		}
	}

	TreeView->ClearExpandedItems();
	ApplyNodeExpansion(Root);
	TreeView->RequestTreeRefresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::ContextMenu_ExpandCriticalPath_CanExecute() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::ContextMenu_ExpandCriticalPath_Execute()
{
	TArray<FTableTreeNodePtr> SelectedNodes;
	if (!TreeView->GetSelectedItems(SelectedNodes))
	{
		return;
	}

	for (const FTableTreeNodePtr& Node : SelectedNodes)
	{
		FTableTreeNodePtr CurrentNode = Node;
		while (CurrentNode->IsGroup())
		{
			check(CurrentNode.IsValid());
			if (!TreeView->IsItemExpanded(CurrentNode))
			{
				CurrentNode->SetExpansion(true);
				TreeView->SetItemExpansion(CurrentNode, true);
			}

			if (CurrentNode->GetFilteredChildren().Num() > 0)
			{
				CurrentNode = StaticCastSharedPtr<FTableTreeNode>(CurrentNode->GetFilteredChildren()[0]);
			}
			else
			{
				break;
			}
		}
	}

	TreeView->RequestTreeRefresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeView::ContextMenu_ExportToFile_CanExecute() const
{
	return !FilteredGroupNodes.IsEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::ContextMenu_ExportToFile_Execute(bool bInExportCollapsed, bool InExportLeafs)
{
	FString DefaultFile = TEXT("Table");
	if (Table.IsValid() && !Table->GetDisplayName().IsEmpty())
	{
		DefaultFile = Table->GetDisplayName().ToString();
		DefaultFile.RemoveSpacesInline();
	}

	TArray<FString> SaveFilenames;
	bool bDialogResult = false;

	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		const FString DefaultPath = FPaths::ProjectSavedDir();
		bDialogResult = DesktopPlatform->SaveFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			LOCTEXT("ExportFileTitle", "Export Table").ToString(),
			DefaultPath,
			DefaultFile,
			TEXT("Comma-Separated Values (*.csv)|*.csv|Tab-Separated Values (*.tsv)|*.tsv|Text Files (*.txt)|*.txt|All Files (*.*)|*.*"),
			EFileDialogFlags::None,
			SaveFilenames
		);
	}

	if (!bDialogResult || SaveFilenames.Num() == 0)
	{
		return;
	}

	FString& Path = SaveFilenames[0];
	TCHAR Separator = TEXT('\t');
	if (Path.EndsWith(TEXT(".csv")))
	{
		Separator = TEXT(',');
	}

	IFileHandle* ExportFileHandle = FPlatformFileManager::Get().GetPlatformFile().OpenWrite(*Path);

	if (ExportFileHandle == nullptr)
	{
		FMessageLog ReportMessageLog((LogListingName != NAME_None) ? LogListingName : TEXT("Other"));
		ReportMessageLog.Error(LOCTEXT("FailedToOpenFile", "Export failed. Failed to open file for write."));
		ReportMessageLog.Notify();
		return;
	}

	FStopwatch Stopwatch;
	Stopwatch.Start();

	UTF16CHAR BOM = UNICODE_BOM;
	ExportFileHandle->Write((uint8*)&BOM, sizeof(UTF16CHAR));

	bool bIncludeHeaders = true;
	auto WriteToFile = [this, &Path, &bIncludeHeaders, ExportFileHandle, Separator](TArray<FBaseTreeNodePtr>& InNodes)
	{
		FString Text;
		this->GetTable()->GetVisibleColumnsData(InNodes, GetLogListingName(), Separator, bIncludeHeaders, Text);
		// Only write the headers once, at the top of the file
		bIncludeHeaders = false;

		// Note: This is a no-op on platforms that are using a 16-bit TCHAR
		FTCHARToUTF16 UTF16String(*Text, Text.Len());
		ExportFileHandle->Write((const uint8*)UTF16String.Get(), UTF16String.Length() * sizeof(UTF16CHAR));

		InNodes.Empty(InNodes.Num());
	};

	TArray<FBaseTreeNodePtr> NodesBuffer;
	ExportToFileRec(Root, NodesBuffer, bInExportCollapsed, InExportLeafs, WriteToFile);

	// Write the remaining lines
	WriteToFile(NodesBuffer);

	ExportFileHandle->Flush();
	delete ExportFileHandle;
	ExportFileHandle = nullptr;

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	UE_LOGF(LogInsights, Log, "Exported table to file in %.3fs.", TotalTime);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::ExportToFileRec(const FBaseTreeNodePtr& InGroupNode, TArray<FBaseTreeNodePtr>& InNodes, bool bInExportCollapsed, bool InExportLeafs, WriteToFileCallback Callback)
{
	constexpr int MaxBufferSize = 10000;

	for (const FBaseTreeNodePtr& Node : InGroupNode->GetFilteredChildren())
	{
		if (InNodes.Num() > MaxBufferSize)
		{
			// Flush the buffer to the file
			Callback(InNodes);
		}

		if (Node->IsGroup() || InExportLeafs)
		{
			InNodes.Add(Node);
		}

		if (Node->IsGroup() && (bInExportCollapsed || TreeView->IsItemExpanded(StaticCastSharedPtr<FTableTreeNode>(Node))))
		{
			ExportToFileRec(Node, InNodes, bInExportCollapsed, InExportLeafs, Callback);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::UpdateBannerText()
{
	TreeViewBannerText = FText::GetEmpty();
	if (TableRowNodes.Num() == 0)
	{
		TreeViewBannerText = LOCTEXT("EmptyTable", "This table is empty.");
	}
	else if (FilteredNodesPtr->Num() == 0)
	{
		TreeViewBannerText = LOCTEXT("AllNodesFilteredOut", "All tree nodes are filtered out. Check the filter configurator.");
	}
	else if (Root->GetFilteredChildrenCount() == 0)
	{
		if (TextFilter->GetRawFilterText().IsEmpty())
		{
			TreeViewBannerText = LOCTEXT("TreeViewIsUpdating", "Tree view is updating. Please wait.");
		}
		else
		{
			TreeViewBannerText = LOCTEXT("HierarchyFilteringZeroResults", "No tree node is matching the current text search.");
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<ITableTreeViewPreset> STableTreeView::GetSelectedViewPreset() const
{
	return SelectedViewPreset;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::SelectViewPreset(TSharedPtr<ITableTreeViewPreset> InPreset)
{
	if (PresetComboBox)
	{
		PresetComboBox->SetSelectedItem(InPreset);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::ApplyViewPreset(const ITableTreeViewPreset& InPreset)
{
	ColumnBeingSorted = InPreset.GetSortColumn();
	ColumnSortMode = InPreset.GetSortMode();
	UpdateCurrentSortingByColumn();

	PreChangeGroupings();
	InPreset.SetCurrentGroupings(AvailableGroupings, CurrentGroupings);
	PostChangeGroupings();

	TArray<FTableColumnConfig > ColumnConfigSet;
	InPreset.GetColumnConfigSet(ColumnConfigSet);
	ApplyColumnConfig(ColumnConfigSet);

	InPreset.OnAppliedToView(*this);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::ApplyColumnConfig(const TArrayView<FTableColumnConfig >& InColumnConfigSet)
{
	// TODO: Reorder columns as in the config set.
	// Currently we only apply visibility and column width.
	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		FTableColumn& Column = ColumnRef.Get();
		const FName ColumnId = Column.GetId();
		const FTableColumnConfig* ConfigPtr = InColumnConfigSet.FindByPredicate([ColumnId](const FTableColumnConfig& Config) { return ColumnId == Config.ColumnId; });
		if (ConfigPtr && ConfigPtr->bIsVisible)
		{
			ShowColumn(Column);
			if (ConfigPtr->Width > 0.0f)
			{
				TreeViewHeaderRow->SetColumnWidth(ColumnId, ConfigPtr->Width);
			}
		}
		else
		{
			HideColumn(Column);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::ViewPreset_OnSelectionChanged(TSharedPtr<ITableTreeViewPreset> InPreset, ESelectInfo::Type SelectInfo)
{
	SelectedViewPreset = InPreset;
	if (InPreset.IsValid())
	{
		ApplyViewPreset(*InPreset);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> STableTreeView::ViewPreset_OnGenerateWidget(TSharedRef<ITableTreeViewPreset> InPreset)
{
	return SNew(STextBlock)
		.Text(InPreset->GetName())
		.ToolTipText(InPreset->GetToolTip())
		.Margin(2.0f);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STableTreeView::ViewPreset_GetSelectedText() const
{
	return SelectedViewPreset ? SelectedViewPreset->GetName() : LOCTEXT("Custom_ToolTip", "Custom");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STableTreeView::ViewPreset_GetSelectedToolTipText() const
{
	return SelectedViewPreset ? SelectedViewPreset->GetToolTip() : LOCTEXT("CustomPreset_ToolTip", "Custom Preset");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void STableTreeView::ConstructViewPreset(TSharedPtr<SHorizontalBox> Box, float MinDesiredWidth)
{
	Box->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(0.0f, 0.0f, 4.0f, 0.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("View", "View:"))
		];

	Box->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.MinDesiredWidth(MinDesiredWidth)
			[
				SAssignNew(PresetComboBox, SComboBox<TSharedRef<ITableTreeViewPreset>>)
				.ToolTipText(this, &STableTreeView::ViewPreset_GetSelectedToolTipText)
				.OptionsSource(GetAvailableViewPresets())
				.OnSelectionChanged(this, &STableTreeView::ViewPreset_OnSelectionChanged)
				.OnGenerateWidget(this, &STableTreeView::ViewPreset_OnGenerateWidget)
				[
					SNew(STextBlock)
					.Text(this, &STableTreeView::ViewPreset_GetSelectedText)
				]
			]
		];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeView::StopAllTableDataTasks(bool bWait)
{
	DataTaskInfos.RemoveAllSwap([](TSharedPtr<FTableTaskInfo> Data) { return Data->Event->IsComplete(); });

	for (int32 Index = 0; Index < DataTaskInfos.Num(); ++Index)
	{
		DataTaskInfos[Index]->CancellationToken->Cancel();
	}

	if (bWait)
	{
		for (int32 Index = 0; Index < DataTaskInfos.Num(); ++Index)
		{
			DataTaskInfos[Index]->Event->Wait();
		}

		DataTaskInfos.Empty();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
