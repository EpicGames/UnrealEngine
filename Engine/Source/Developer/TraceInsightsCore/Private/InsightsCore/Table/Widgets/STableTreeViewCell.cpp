// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Table/Widgets/STableTreeViewCell.h"

#include "SlateOptMacros.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SExpanderArrow.h"

#include "InsightsCore/Common/InsightsCoreStyle.h"
#include "InsightsCore/Table/ViewModels/Table.h"
#include "InsightsCore/Table/ViewModels/TableCellValueFormatter.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"
#include "InsightsCore/Table/Widgets/STableTreeViewRow.h"

#define LOCTEXT_NAMESPACE "UE::Insights::STableTreeView"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void STableTreeViewCell::Construct(const FArguments& InArgs, const TSharedRef<ITableRow>& InTableRow)
{
	WeakTableRow = InTableRow.ToWeakPtr();

	TablePtr = InArgs._TablePtr;
	ColumnPtr = InArgs._ColumnPtr;
	TableTreeNodePtr = InArgs._TableTreeNodePtr;

	ensure(TablePtr.IsValid());
	ensure(ColumnPtr.IsValid());
	ensure(TableTreeNodePtr.IsValid());

	SetHoveredCellDelegate = InArgs._OnSetHoveredCell;

	ChildSlot
	[
		GenerateWidgetForColumn(InArgs)
	];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STableTreeViewCell::GenerateWidgetForColumn(const FArguments& InArgs)
{
	if (InArgs._ColumnPtr.IsValid() && InArgs._ColumnPtr->IsHierarchy())
	{
		return GenerateWidgetForNameColumn(InArgs);
	}
	else
	{
		return GenerateWidgetForTableColumn(InArgs);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STableTreeViewCell::GenerateWidgetForNameColumn(const FArguments& InArgs)
{
	TSharedPtr<IToolTip> RowToolTip;
	TSharedPtr<ITableRow> TableRow = WeakTableRow.Pin();
	if (TableRow.IsValid())
	{
		TSharedPtr<STableTreeViewRow> Row = StaticCastSharedPtr<STableTreeViewRow, ITableRow>(TableRow);
		RowToolTip = Row->GetRowToolTip();
	}

	return
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(SExpanderArrow, TableRow)
		]

		// Icon + tooltip
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(this, &STableTreeViewCell::GetIcon)
			.ColorAndOpacity(this, &STableTreeViewCell::GetIconColorAndOpacity)
			.ToolTip(RowToolTip)
		]

		// Name
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(FMargin(2.0f, 0.0f, 2.0f, 0.0f))
		[
			SNew(STextBlock)
			.Text(this, &STableTreeViewCell::GetDisplayName)
			.HighlightText(InArgs._HighlightText)
			.TextStyle(FInsightsCoreStyle::Get(), TEXT("TreeTable.NameText"))
			.ColorAndOpacity(this, &STableTreeViewCell::GetDisplayNameColorAndOpacity)
			.ShadowColorAndOpacity(this, &STableTreeViewCell::GetShadowColorAndOpacity)
		]

		// Name Suffix
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(FMargin(2.0f, 0.0f, 2.0f, 0.0f))
		[
			SNew(STextBlock)
			.Visibility(this, &STableTreeViewCell::HasExtraDisplayName)
			.Text(this, &STableTreeViewCell::GetExtraDisplayName)
			.TextStyle(FInsightsCoreStyle::Get(), TEXT("TreeTable.NameText"))
			.ColorAndOpacity(this, &STableTreeViewCell::GetExtraDisplayNameColorAndOpacity)
			.ShadowColorAndOpacity(this, &STableTreeViewCell::GetShadowColorAndOpacity)
		]
	;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STableTreeViewCell::GetValueAsText() const
{
	return ColumnPtr->GetValueAsText(*TableTreeNodePtr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STableTreeViewCell::GenerateWidgetForTableColumn(const FArguments& InArgs)
{
	TSharedPtr<SWidget> CustomWidget = ColumnPtr->GetValueFormatter()->GenerateCustomWidget(*ColumnPtr, *TableTreeNodePtr);
	if (CustomWidget.IsValid())
	{
		return CustomWidget.ToSharedRef();
	}

	TSharedRef<STextBlock> TextBox = SNew(STextBlock)
		.TextStyle(FInsightsCoreStyle::Get(), TEXT("TreeTable.NormalText"))
		.ColorAndOpacity(this, &STableTreeViewCell::GetNormalTextColorAndOpacity)
		.ShadowColorAndOpacity(this, &STableTreeViewCell::GetShadowColorAndOpacity);

	if (ColumnPtr->IsDynamic())
	{
		TextBox->SetText(TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateRaw(this, &STableTreeViewCell::GetValueAsText)));
	}
	else
	{
		FText CellText = ColumnPtr->GetValueAsText(*TableTreeNodePtr);
		TextBox->SetText(CellText);
	}

	TSharedPtr<IToolTip> ColumnToolTip = ColumnPtr->GetValueFormatter()->GetCustomTooltip(*ColumnPtr, *TableTreeNodePtr);

	return SNew(SBox)
		.ToolTip(ColumnToolTip)
		.HAlign(ColumnPtr->GetHorizontalAlignment())
		.VAlign(VAlign_Center)
		.Padding(FMargin(2.0f, 0.0f, 2.0f, 0.0f))
		[
			TextBox
		];
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeViewCell::IsSelected() const
{
	TSharedPtr<ITableRow> TableRow = WeakTableRow.Pin();
	return TableRow.IsValid() && TableRow->IsItemSelected();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor STableTreeViewCell::GetIconColorAndOpacity() const
{
	bool bIsHoveredOrSelected = IsHovered() || IsSelected();

	return bIsHoveredOrSelected ?
		TableTreeNodePtr->GetIconColor() :
		TableTreeNodePtr->GetIconColor().CopyWithNewOpacity(0.8f);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor STableTreeViewCell::GetDisplayNameColorAndOpacity() const
{
	bool bIsHoveredOrSelected = IsHovered() || IsSelected();

	if (TableTreeNodePtr->IsFiltered())
	{
		return bIsHoveredOrSelected ?
			TableTreeNodePtr->GetColor().CopyWithNewOpacity(0.5f) :
			TableTreeNodePtr->GetColor().CopyWithNewOpacity(0.4f);
	}
	else
	{
		return bIsHoveredOrSelected ?
			TableTreeNodePtr->GetColor() :
			TableTreeNodePtr->GetColor().CopyWithNewOpacity(0.8f);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor STableTreeViewCell::GetExtraDisplayNameColorAndOpacity() const
{
	bool bIsHoveredOrSelected = IsHovered() || IsSelected();

	if (TableTreeNodePtr->IsFiltered())
	{
		return bIsHoveredOrSelected ?
			FLinearColor(0.3f, 0.3f, 0.3f, 0.5f) :
			FLinearColor(0.3f, 0.3f, 0.3f, 0.4f);
	}
	else
	{
		return bIsHoveredOrSelected ?
			FLinearColor(0.3f, 0.3f, 0.3f, 1.0f) :
			FLinearColor(0.3f, 0.3f, 0.3f, 0.8f);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor STableTreeViewCell::GetNormalTextColorAndOpacity() const
{
	bool bIsHoveredOrSelected = IsHovered() || IsSelected();

	if (TableTreeNodePtr->IsGroup())
	{
		if (TableTreeNodePtr->IsFiltered())
		{
			return bIsHoveredOrSelected ?
				FStyleColors::ForegroundHover.GetSpecifiedColor().CopyWithNewOpacity(0.4f) :
				FStyleColors::Foreground.GetSpecifiedColor().CopyWithNewOpacity(0.4f);
		}
		else
		{
			return bIsHoveredOrSelected ?
				FStyleColors::ForegroundHover.GetSpecifiedColor().CopyWithNewOpacity(0.8f) :
				FStyleColors::Foreground.GetSpecifiedColor().CopyWithNewOpacity(0.8f);
		}
	}
	else
	{
		if (TableTreeNodePtr->IsFiltered())
		{
			return bIsHoveredOrSelected ?
				FStyleColors::ForegroundHover.GetSpecifiedColor().CopyWithNewOpacity(0.5f) :
				FStyleColors::Foreground.GetSpecifiedColor().CopyWithNewOpacity(0.5f);
		}
		else
		{
			return bIsHoveredOrSelected ?
				FStyleColors::ForegroundHover :
				FStyleColors::Foreground;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor STableTreeViewCell::GetShadowColorAndOpacity() const
{
	const FLinearColor ShadowColor =
		TableTreeNodePtr->IsFiltered() ?
		FLinearColor(0.0f, 0.0f, 0.0f, 0.25f) :
		FLinearColor(0.0f, 0.0f, 0.0f, 0.5f);
	return ShadowColor;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
