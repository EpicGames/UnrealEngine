// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Table/Widgets/STableTreeViewRow.h"

#include "SlateOptMacros.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Images/SImage.h"

#include "InsightsCore/Common/InsightsCoreStyle.h"
#include "InsightsCore/Common/TimeUtils.h"
#include "InsightsCore/Table/ViewModels/Table.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"
#include "InsightsCore/Table/Widgets/STableTreeViewCell.h"
#include "InsightsCore/Table/Widgets/STableTreeViewTooltip.h"

#define LOCTEXT_NAMESPACE "UE::Insights::STableTreeView"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeViewRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	OnShouldBeEnabled = InArgs._OnShouldBeEnabled;
	IsColumnVisibleDelegate = InArgs._OnIsColumnVisible;
	GetColumnOutlineHAlignmentDelegate = InArgs._OnGetColumnOutlineHAlignmentDelegate;
	SetHoveredCellDelegate = InArgs._OnSetHoveredCell;

	HighlightText = InArgs._HighlightText;

	TablePtr = InArgs._TablePtr;
	TableTreeNodePtr = InArgs._TableTreeNodePtr;

	RowToolTip = MakeShared<STableTreeRowToolTip>(TableTreeNodePtr);

	SetEnabled(TAttribute<bool>(this, &STableTreeViewRow::HandleShouldBeEnabled));

	SMultiColumnTableRow<FTableTreeNodePtr>::Construct(SMultiColumnTableRow<FTableTreeNodePtr>::FArguments(), InOwnerTableView);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> STableTreeViewRow::GenerateWidgetForColumn(const FName& ColumnId)
{
	return SNew(SOverlay)
		.Visibility(EVisibility::SelfHitTestInvisible)

		+ SOverlay::Slot()
		.Padding(0.0f)
		[
			SNew(SImage)
			.Image(FInsightsCoreStyle::GetBrush("TreeTable.RowBackground"))
			.ColorAndOpacity(this, &STableTreeViewRow::GetBackgroundColorAndOpacity)
		]

		+ SOverlay::Slot()
		.Padding(0.0f)
		[
			SNew(SImage)
			.Image(this, &STableTreeViewRow::GetOutlineBrush, ColumnId)
			.ColorAndOpacity(this, &STableTreeViewRow::GetOutlineColorAndOpacity)
		]

		+ SOverlay::Slot()
		[
			CreateCellWidget(ColumnId)
		];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> STableTreeViewRow::CreateCellWidget(FName ColumnId)
{
	TSharedRef<FTableColumn> ColumnPtr = TablePtr->FindColumnChecked(ColumnId);

	return SNew(STableTreeViewCell, SharedThis(this))
		.Visibility(this, &STableTreeViewRow::IsColumnVisible, ColumnId)
		.TablePtr(TablePtr)
		.ColumnPtr(ColumnPtr.ToSharedPtr())
		.TableTreeNodePtr(TableTreeNodePtr)
		.HighlightText(HighlightText)
		.OnSetHoveredCell(this, &STableTreeViewRow::OnSetHoveredCell);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STableTreeViewRow::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return SMultiColumnTableRow<FTableTreeNodePtr>::OnDragDetected(MyGeometry, MouseEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<IToolTip> STableTreeViewRow::GetRowToolTip() const
{
	return RowToolTip.ToSharedRef();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeViewRow::InvalidateContent()
{
	RowToolTip->InvalidateWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor STableTreeViewRow::GetBackgroundColorAndOpacity() const
{
	return FLinearColor(0.0f, 0.0f, 0.0f, 1.0f);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor STableTreeViewRow::GetBackgroundColorAndOpacity(double Time) const
{
	const FLinearColor Color =	Time > FTimeValue::Second      ? FLinearColor(0.3f, 0.0f, 0.0f, 1.0f) :
								Time > FTimeValue::Millisecond ? FLinearColor(0.3f, 0.1f, 0.0f, 1.0f) :
								Time > FTimeValue::Microsecond ? FLinearColor(0.0f, 0.1f, 0.0f, 1.0f) :
								                                 FLinearColor(0.0f, 0.0f, 0.0f, 1.0f);
	return Color;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor STableTreeViewRow::GetOutlineColorAndOpacity() const
{
	return FLinearColor(0.0f, 0.0f, 0.0f, 0.0f);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* STableTreeViewRow::GetOutlineBrush(const FName ColumnId) const
{
	EHorizontalAlignment HAlign = HAlign_Center;
	if (GetColumnOutlineHAlignmentDelegate.IsBound())
	{
		HAlign = GetColumnOutlineHAlignmentDelegate.Execute(ColumnId);
	}
	return FInsightsCoreStyle::GetOutlineBrush(HAlign);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STableTreeViewRow::HandleShouldBeEnabled() const
{
	if (TableTreeNodePtr->IsGroup())
	{
		return true;
	}
	else if (OnShouldBeEnabled.IsBound())
	{
		return OnShouldBeEnabled.Execute(TableTreeNodePtr);
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EVisibility STableTreeViewRow::IsColumnVisible(const FName ColumnId) const
{
	EVisibility Result = EVisibility::Collapsed;
	if (IsColumnVisibleDelegate.IsBound())
	{
		Result = IsColumnVisibleDelegate.Execute(ColumnId) ? EVisibility::Visible : EVisibility::Collapsed;
	}
	return Result;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeViewRow::OnSetHoveredCell(TSharedPtr<FTable> InTablePtr, TSharedPtr<FTableColumn> InColumnPtr, FTableTreeNodePtr InTreeNodePtr)
{
	SetHoveredCellDelegate.ExecuteIfBound(InTablePtr, InColumnPtr, InTreeNodePtr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
