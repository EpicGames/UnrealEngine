// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Table/Widgets/STableTreeViewTooltip.h"

#include "SlateOptMacros.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

#include "InsightsCore/Common/InsightsCoreStyle.h"
#include "InsightsCore/Common/TimeUtils.h"
#include "InsightsCore/Table/ViewModels/Table.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"
#include "InsightsCore/Table/ViewModels/TableTreeNode.h"

#define LOCTEXT_NAMESPACE "UE::Insights::STableTreeView"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

static const FTextBlockStyle* GetTableTreeViewTooltipHeaderTextStyle()
{
	return &FInsightsCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>(TEXT("TreeTable.TooltipBold"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

static const FTextBlockStyle* GetTableTreeViewTooltipNormalTextStyle()
{
	return &FInsightsCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>(TEXT("TreeTable.Tooltip"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

TSharedPtr<SToolTip> STableTreeViewTooltip::GetTableTooltip(const FTable& Table)
{
	TSharedPtr<SToolTip> ColumnTooltip =
		SNew(SToolTip)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(Table.GetDisplayName())
				.TextStyle(GetTableTreeViewTooltipHeaderTextStyle())
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(Table.GetDescription())
				.TextStyle(GetTableTreeViewTooltipNormalTextStyle())
			]
		];

	return ColumnTooltip;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SToolTip> STableTreeViewTooltip::GetColumnTooltip(const FTableColumn& Column)
{
	TSharedPtr<SToolTip> ColumnTooltip =
		SNew(SToolTip)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(Column.GetTitleName())
				.TextStyle(GetTableTreeViewTooltipHeaderTextStyle())
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(Column.GetDescription())
				.TextStyle(GetTableTreeViewTooltipNormalTextStyle())
			]
		];

	return ColumnTooltip;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SToolTip> STableTreeViewTooltip::GetRowTooltip(const TSharedPtr<FTableTreeNode> TreeNodePtr)
{
	TSharedPtr<SGridPanel> GridPanel;
	TSharedPtr<SHorizontalBox> HBox;

	const FText NodeTooltip = TreeNodePtr->GetTooltipText();
	const EVisibility NodeTooltipVisibility = NodeTooltip.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;

	TSharedPtr<SToolTip> TableCellTooltip =
		SNew(SToolTip)
		[
			SAssignNew(HBox, SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SGridPanel)

					// Row: [RowIndex]
					+ SGridPanel::Slot(0, 0)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TT_Id", "Row:"))
						.TextStyle(GetTableTreeViewTooltipHeaderTextStyle())
					]
					+ SGridPanel::Slot(1, 0)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(FText::AsNumber(TreeNodePtr->GetRowIndex()))
						.TextStyle(GetTableTreeViewTooltipNormalTextStyle())
					]

					// Item Type: [Type]
					+ SGridPanel::Slot(0, 1)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TT_Type", "Item Type:"))
						.TextStyle(GetTableTreeViewTooltipHeaderTextStyle())
					]
					+ SGridPanel::Slot(1, 1)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(TreeNodePtr->IsGroup() ? LOCTEXT("TT_Type_Group", "Group Node") : LOCTEXT("TT_Type_TableRow", "Table Row"))
						.TextStyle(GetTableTreeViewTooltipNormalTextStyle())
					]

					// Name: [Name]
					+ SGridPanel::Slot(0, 2)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TT_Name", "Name:"))
						.TextStyle(GetTableTreeViewTooltipHeaderTextStyle())
					]
					+ SGridPanel::Slot(1, 2)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.WrapTextAt(512.0f)
						.WrappingPolicy(ETextWrappingPolicy::AllowPerCharacterWrapping)
						.Text(TreeNodePtr->GetDisplayName())
						.TextStyle(GetTableTreeViewTooltipNormalTextStyle())
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SSeparator)
					.Visibility(NodeTooltipVisibility)
					.Orientation(Orient_Horizontal)
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(STextBlock)
					.Visibility(NodeTooltipVisibility)
					.WrapTextAt(1024.0f)
					.WrappingPolicy(ETextWrappingPolicy::AllowPerCharacterWrapping)
					.Text(NodeTooltip)
					.TextStyle(GetTableTreeViewTooltipNormalTextStyle())
					.ColorAndOpacity(FLinearColor(0.5f, 0.5f, 0.5f, 1.0f))
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SAssignNew(GridPanel, SGridPanel)

					// Values for each table column are added here.
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
				]
			]
		];

	TSharedPtr<FTable> Table = TreeNodePtr->GetParentTable().Pin();
	if (Table.IsValid())
	{
		int32 Row = 0;
		for (const TSharedRef<FTableColumn>& Column : Table->GetColumns())
		{
			if (!Column->IsHierarchy())
			{
				const FText Value = Column->GetValueAsTooltipText(*TreeNodePtr);
				if (!Value.IsEmpty())
				{
					const FText Name = FText::Format(LOCTEXT("TooltipValueFormat", "{0}:"), Column->GetTitleName());
					AddGridRow(GridPanel, Row, Name, Value);
				}
			}
		}
	}

	return TableCellTooltip;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STableTreeViewTooltip::AddGridRow(TSharedPtr<SGridPanel> Grid, int32& Row, const FText& Name, const FText& Value)
{
	Grid->AddSlot(0, Row)
		.Padding(2.0f)
		[
			SNew(STextBlock)
			.Text(Name)
			.TextStyle(GetTableTreeViewTooltipHeaderTextStyle())
		];

	Grid->AddSlot(1, Row)
		.Padding(2.0f)
		[
			SNew(STextBlock)
			.Text(Value)
			.TextStyle(GetTableTreeViewTooltipNormalTextStyle())
		];

	Row++;
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
