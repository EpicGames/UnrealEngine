// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Table/Widgets/SUntypedDiffTableTreeView.h"

#include "SlateOptMacros.h"
#include "Logging/MessageLog.h"

// TraceServices
#include "TraceServices/Model/TableMerge.h"

#define LOCTEXT_NAMESPACE "UE::Insights::SUntypedDiffTableTreeView"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

void SUntypedDiffTableTreeView::UpdateSourceTableA(const FString& Name, TSharedPtr<TraceServices::IUntypedTable> SourceTable)
{
	TableNameA = Name;
	SourceTableA = SourceTable;

	if (SourceTableA.IsValid() && SourceTableB.IsValid())
	{
		RequestMergeTables();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SUntypedDiffTableTreeView::UpdateSourceTableB(const FString& Name, TSharedPtr<TraceServices::IUntypedTable> SourceTable)
{
	TableNameB = Name;
	SourceTableB = SourceTable;

	if (SourceTableA.IsValid() && SourceTableB.IsValid())
	{
		RequestMergeTables();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SUntypedDiffTableTreeView::SwapTables_OnClicked()
{
	Swap(TableNameA, TableNameB);
	Swap(SourceTableA, SourceTableB);

	if (SourceTableA.IsValid() && SourceTableB.IsValid())
	{
		RequestMergeTables();
	}

	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SUntypedDiffTableTreeView::GetSwapButtonText() const
{
	const FText CleanA = FText::FromString(FPaths::GetCleanFilename(TableNameA));
	const FText CleanB = FText::FromString(FPaths::GetCleanFilename(TableNameB));
	return FText::Format(LOCTEXT("ToolBar_SwapButtonText", "SWAP (A: {0} - B: {1})"), CleanA, CleanB);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedPtr<SWidget> SUntypedDiffTableTreeView::ConstructToolbar()
{
	return
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			SNew(SButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("PrimaryButton"))
			.ForegroundColor(FSlateColor::UseStyle())
			.HAlign(HAlign_Center)
			.Text(this, &SUntypedDiffTableTreeView::GetSwapButtonText)
			.OnClicked(this, &SUntypedDiffTableTreeView::SwapTables_OnClicked)
		];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SUntypedDiffTableTreeView::RequestMergeTables()
{
	SetCurrentOperationNameOverride(LOCTEXT("TableDiffMessage", "Merging Tables"));

	// Clear current table
	if (TreeView.IsValid())
	{
		TreeView->SetTreeItemsSource(&DummyGroupNodes);
		TreeView->RequestTreeRefresh();
	}

	TraceServices::FTableMergeService::MergeTables(SourceTableA, SourceTableB,
		[this](TSharedPtr<TraceServices::FTableDiffCallbackParams> Params)
		{
			ClearCurrentOperationNameOverride();
			if (Params->Result == TraceServices::ETableDiffResult::ESuccess)
			{
				this->UpdateSourceTable(Params->Table);
			}
			else
			{
				FName LogListingName(TEXT("Insights"));
				FMessageLog ReportMessageLog(LogListingName);
				ReportMessageLog.AddMessages(Params->Messages);
				ReportMessageLog.Notify();
			}
		});
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
