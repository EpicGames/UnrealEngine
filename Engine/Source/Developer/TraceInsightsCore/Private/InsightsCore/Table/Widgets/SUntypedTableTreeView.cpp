// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Table/Widgets/SUntypedTableTreeView.h"

// TraceServices
#include "TraceServices/Containers/Tables.h"

// TraceInsightsCore
#include "InsightsCore/Common/Log.h"
#include "InsightsCore/Common/Stopwatch.h"
#include "InsightsCore/Table/ViewModels/UntypedTable.h"

#define LOCTEXT_NAMESPACE "UE::Insights::SUntypedTableTreeView"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

SUntypedTableTreeView::SUntypedTableTreeView()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SUntypedTableTreeView::~SUntypedTableTreeView()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SUntypedTableTreeView::Construct(const FArguments& InArgs, TSharedPtr<FUntypedTable> InTablePtr)
{
	bRunInAsyncMode = InArgs._RunInAsyncMode;
	ConstructWidget(InTablePtr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SUntypedTableTreeView::Reset()
{
	STableTreeView::Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SUntypedTableTreeView::UpdateSourceTable(TSharedPtr<TraceServices::IUntypedTable> SourceTable)
{
	//check(Table->Is<FUntypedTable>());
	TSharedPtr<FUntypedTable> UntypedTable = StaticCastSharedPtr<FUntypedTable>(Table);

	if (UntypedTable->UpdateSourceTable(SourceTable))
	{
		RebuildColumns();
	}

	RebuildTree(true);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SUntypedTableTreeView::RebuildTree(bool bResync)
{
	FStopwatch Stopwatch;
	Stopwatch.Start();

	FStopwatch SyncStopwatch;
	SyncStopwatch.Start();

	if (bResync)
	{
		TableRowNodes.Empty();
	}

	const int32 PreviousNodeCount = TableRowNodes.Num();

	//check(Table->Is<FUntypedTable>());
	TSharedPtr<FUntypedTable> UntypedTable = StaticCastSharedPtr<FUntypedTable>(Table);

	TSharedPtr<TraceServices::IUntypedTable> SourceTable = UntypedTable->GetSourceTable();
	TSharedPtr<TraceServices::IUntypedTableReader> TableReader = UntypedTable->GetTableReader();

	if (SourceTable.IsValid() && TableReader.IsValid())
	{
		const int32 TotalRowCount = static_cast<int32>(SourceTable->GetRowCount());
		if (TotalRowCount != TableRowNodes.Num())
		{
			TableRowNodes.Empty(TotalRowCount);
			for (int32 RowIndex = 0; RowIndex < TotalRowCount; ++RowIndex)
			{
				TableReader->SetRowIndex(RowIndex);
				FTableTreeNodePtr NodePtr = MakeShared<FTableTreeNode>(Table, RowIndex);
				TableRowNodes.Add(NodePtr);
			}
			ensure(TableRowNodes.Num() == TotalRowCount);
		}
	}

	SyncStopwatch.Stop();

	if (bResync || TableRowNodes.Num() != PreviousNodeCount)
	{
		// Save selection.
		TArray<FTableTreeNodePtr> SelectedItems;
		TreeView->GetSelectedItems(SelectedItems);

		UpdateTree();

		TreeView->RebuildList();

		// Restore selection.
		if (SelectedItems.Num() > 0)
		{
			TreeView->ClearSelection();
			for (FTableTreeNodePtr& NodePtr : SelectedItems)
			{
				NodePtr = GetNodeByTableRowIndex(NodePtr->GetRowIndex());
			}
			SelectedItems.RemoveAll([](const FTableTreeNodePtr& NodePtr) { return !NodePtr.IsValid(); });
			if (SelectedItems.Num() > 0)
			{
				TreeView->SetItemSelection(SelectedItems, true);
				TreeView->RequestScrollIntoView(SelectedItems.Last());
			}
		}
	}

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	if (TotalTime > 0.01)
	{
		const double SyncTime = SyncStopwatch.GetAccumulatedTime();
		UE_LOGF(LogInsights, Log, "[Table] Tree view rebuilt in %.4fs (sync: %.4fs + update: %.4fs) --> %d rows (%d added)",
			TotalTime, SyncTime, TotalTime - SyncTime, TableRowNodes.Num(), TableRowNodes.Num() - PreviousNodeCount);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SUntypedTableTreeView::IsRunning() const
{
	return !CurrentOperationNameOverride.IsEmpty() || STableTreeView::IsRunning();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

double SUntypedTableTreeView::GetAllOperationsDuration()
{
	if (!CurrentOperationNameOverride.IsEmpty())
	{
		CurrentOperationStopwatch.Update();
		return CurrentOperationStopwatch.GetAccumulatedTime();
	}

	return STableTreeView::GetAllOperationsDuration();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SUntypedTableTreeView::GetCurrentOperationName() const
{
	if (!CurrentOperationNameOverride.IsEmpty())
	{
		return CurrentOperationNameOverride;
	}

	return STableTreeView::GetCurrentOperationName();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SUntypedTableTreeView::SetCurrentOperationNameOverride(const FText& InOperationName)
{
	CurrentOperationStopwatch.Start();
	CurrentOperationNameOverride = InOperationName;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SUntypedTableTreeView::ClearCurrentOperationNameOverride()
{
	CurrentOperationStopwatch.Stop();
	CurrentOperationStopwatch.Reset();

	CurrentOperationNameOverride = FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
