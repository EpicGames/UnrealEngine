// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceInsightsCoreModule.h"

#include "Modules/ModuleManager.h"

#include "InsightsCore/Common/InsightsCoreStyle.h"
#include "InsightsCore/Filter/ViewModels/Filters.h"

IMPLEMENT_MODULE(FTraceInsightsCoreModule, TraceInsightsCore);

void FTraceInsightsCoreModule::StartupModule()
{
	UE::Insights::FInsightsCoreStyle::Initialize();
	UE::Insights::FFilterService::Initialize();
}

void FTraceInsightsCoreModule::ShutdownModule()
{
	UE::Insights::FFilterService::Shutdown();
	UE::Insights::FInsightsCoreStyle::Shutdown();
}
