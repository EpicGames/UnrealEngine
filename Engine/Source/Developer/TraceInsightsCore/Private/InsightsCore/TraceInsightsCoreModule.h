// Copyright Epic Games, Inc. All Rights Reserved

#pragma once

#include "InsightsCore/ITraceInsightsCoreModule.h"

class FTraceInsightsCoreModule : public ITraceInsightsCoreModule
{
public:
	virtual ~FTraceInsightsCoreModule() = default;

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
