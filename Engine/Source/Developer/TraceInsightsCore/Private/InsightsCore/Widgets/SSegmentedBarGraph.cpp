// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsCore/Widgets/SSegmentedBarGraph.h"

#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "UE::Insights::SSegmentedBarGraph"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

void FObservableSegmentArrayPointer::HandleArrayChanged(typename ::UE::Slate::Containers::TObservableArray<FObservableSegmentArrayPointer::SegmentType>::ObservableArrayChangedArgsType Args)
{
	if (TSharedPtr<WidgetType> GraphWidgetOwnerPin = GraphWidgetOwner.Pin())
	{
		GraphWidgetOwnerPin->RequestGraphRefresh();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSharedObservableSegmentArray::HandleArrayChanged(typename ::UE::Slate::Containers::TObservableArray<FSharedObservableSegmentArray::SegmentType>::ObservableArrayChangedArgsType Args)
{
	if (TSharedPtr<SSegmentedBarGraph> GraphWidgetOwnerPin = GraphWidgetOwner.Pin())
	{
		GraphWidgetOwnerPin->RequestGraphRefresh();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// SSegmentedBarGraph
////////////////////////////////////////////////////////////////////////////////////////////////////

SSegmentedBarGraph::SSegmentedBarGraph()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SSegmentedBarGraph::~SSegmentedBarGraph()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SSegmentedBarGraph::Construct(const FArguments& InArgs)
{
	this->SetSegmentsSource(InArgs.MakeSegmentsSource(this->SharedThis(this)));

	ChildSlot
		[
			SNew(SBorder)
			.Padding(0.0f)
			.BorderImage(FAppStyle::Get().GetBrush("WhiteBrush"))
			.BorderBackgroundColor(FSlateColor(EStyleColor::Background))
			[
				SAssignNew(HorizontalBox, SHorizontalBox)
			]
		];

	RequestGraphRefresh();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SSegmentedBarGraph::RequestGraphRefresh()
{
	if (HorizontalBox.IsValid())
	{
		HorizontalBox->ClearChildren();

		TArrayView<const TSharedPtr<IBarGraphSegment>> Segments = GetSegments();
		for (int32 SegmentIndex = 0; SegmentIndex < Segments.Num(); ++SegmentIndex)
		{
			AddHorizontalSegment(HorizontalBox.ToSharedRef(), SegmentIndex);
		}

		HorizontalBox->Invalidate(EInvalidateWidgetReason::ChildOrder);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SSegmentedBarGraph::AddHorizontalSegment(TSharedRef<SHorizontalBox> Box, int32 SegmentIndex)
{
	Box->AddSlot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(FMargin(1.0f, 1.0f))
		.FillWidth(TAttribute<float>::CreateSP(this, &SSegmentedBarGraph::GetSegmentSize, SegmentIndex))
		[
			SNew(SBorder)
			.Padding(0.0f)
			.BorderImage(FAppStyle::Get().GetBrush("WhiteBrush"))
			.BorderBackgroundColor(this, &SSegmentedBarGraph::GetSegmentColor, SegmentIndex)
			.ToolTipText(this, &SSegmentedBarGraph::GetSegmentToolTipText, SegmentIndex)
			[
				SNew(SBox)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Margin(FMargin(2.0f, 0.0f))
					.ColorAndOpacity(this, &SSegmentedBarGraph::GetSegmentTextColor, SegmentIndex)
					.Text(this, &SSegmentedBarGraph::GetSegmentText, SegmentIndex)
					.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
				]
			]
		];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

float SSegmentedBarGraph::GetSegmentSize(int32 SegmentIndex) const
{
	TArrayView<const TSharedPtr<IBarGraphSegment>> Segments = GetSegments();
	if (SegmentIndex < 0 || SegmentIndex >= Segments.Num())
	{
		return 0;
	}
	return static_cast<float>(Segments[SegmentIndex]->GetSize());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SSegmentedBarGraph::GetSegmentText(int32 SegmentIndex) const
{
	TArrayView<const TSharedPtr<IBarGraphSegment>> Segments = GetSegments();
	if (SegmentIndex < 0 || SegmentIndex >= Segments.Num())
	{
		return FText::GetEmpty();
	}
	return Segments[SegmentIndex]->GetText();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SSegmentedBarGraph::GetSegmentToolTipText(int32 SegmentIndex) const
{
	TArrayView<const TSharedPtr<IBarGraphSegment>> Segments = GetSegments();
	if (SegmentIndex < 0 || SegmentIndex >= Segments.Num())
	{
		return FText::GetEmpty();
	}
	return Segments[SegmentIndex]->GetToolTipText();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor SSegmentedBarGraph::GetSegmentColor(int32 SegmentIndex) const
{
	TArrayView<const TSharedPtr<IBarGraphSegment>> Segments = GetSegments();
	if (SegmentIndex < 0 || SegmentIndex >= Segments.Num())
	{
		FLinearColor Color(0.75f, 0.75f, 0.75f, 1.0f);
		return FSlateColor(Color);
	}
	return FSlateColor(Segments[SegmentIndex]->GetColor());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor SSegmentedBarGraph::GetSegmentTextColor(int32 SegmentIndex) const
{
	TArrayView<const TSharedPtr<IBarGraphSegment>> Segments = GetSegments();
	if (SegmentIndex < 0 || SegmentIndex >= Segments.Num())
	{
		FLinearColor Color(0.0f, 0.0f, 0.0f, 1.0f);
		return FSlateColor(Color);
	}
	return FSlateColor(Segments[SegmentIndex]->GetTextColor());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
