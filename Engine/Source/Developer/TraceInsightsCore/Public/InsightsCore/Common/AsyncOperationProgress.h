// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include <atomic>

namespace UE::Insights
{

class IAsyncOperationProgress
{
public:
	virtual void CancelCurrentAsyncOp() = 0;
	virtual void Reset() = 0;
	virtual bool ShouldCancelAsyncOp() const = 0;
};

class FAsyncOperationProgress : public IAsyncOperationProgress
{
public:
	virtual void CancelCurrentAsyncOp() override { bCancelCurrentAsyncOp.store(true); }
	virtual void Reset() override{ bCancelCurrentAsyncOp.store(false); }
	virtual bool ShouldCancelAsyncOp() const override { return bCancelCurrentAsyncOp.load(); }

private:
	std::atomic<bool> bCancelCurrentAsyncOp{ false };
};

} // namespace UE::Insights
