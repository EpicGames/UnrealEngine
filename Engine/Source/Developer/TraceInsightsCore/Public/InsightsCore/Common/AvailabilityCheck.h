// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#define UE_API TRACEINSIGHTSCORE_API

namespace UE::Insights
{

/**
 * Utility class used by profiler managers to limit how often they check for availability conditions.
 */
class FAvailabilityCheck
{
public:
	/** Returns true if managers are allowed to do (slow) availability check during this tick. */
	UE_API bool Tick();

	/** Disables the "availability check" (i.e. Tick() calls will return false when disabled). */
	UE_API void Disable();

	/** Enables the "availability check" with a specified initial delay. */
	UE_API void Enable(double InWaitTime);

private:
	double WaitTime = 0.0;
	uint64 NextTimestamp = (uint64)-1;
};

} // namespace UE::Insights

#undef UE_API
