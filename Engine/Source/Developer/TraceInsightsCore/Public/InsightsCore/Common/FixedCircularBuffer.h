// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace UE::Insights
{

template <typename T, int32 Size = 64>
struct TFixedCircularBuffer
{
	T Buffer[Size];
	int32 BufferStartOffset = 0;
	int32 BufferLength = 0;

	void Reset()
	{
		BufferStartOffset = 0;
		BufferLength = 0;
	}

	void AddValue(const T Value)
	{
		Buffer[BufferStartOffset] = Value;
		BufferStartOffset++;
		if (BufferStartOffset == Size)
		{
			BufferStartOffset = 0;
		}
		if (BufferLength < Size)
		{
			BufferLength++;
		}
	}

	const T GetValue(const int32 RecentIndex) const
	{
		return Buffer[(BufferStartOffset + Size - RecentIndex - 1) % Size];
	}

	const T ComputeAverage() const
	{
		if (BufferLength == 0)
		{
			return 0;
		}
		T Total = 0;
		for (int32 RecentIndex = 0; RecentIndex < BufferLength; ++RecentIndex)
		{
			Total += Buffer[(BufferStartOffset + Size - RecentIndex - 1) % Size];
		}
		return Total / BufferLength;
	}
};

} // namespace UE::Insights
