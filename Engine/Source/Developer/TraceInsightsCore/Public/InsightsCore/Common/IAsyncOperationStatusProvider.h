// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

class FText;

namespace UE::Insights
{

class IAsyncOperationStatusProvider
{
public:
	virtual bool IsRunning() const = 0;

	virtual double GetAllOperationsDuration() = 0;
	virtual double GetCurrentOperationDuration() = 0;
	virtual uint32 GetOperationCount() const = 0;

	virtual FText GetCurrentOperationName() const = 0;
};

} // namespace UE::Insights
