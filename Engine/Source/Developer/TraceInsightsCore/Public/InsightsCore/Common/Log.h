// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/LowLevelMemTracker.h"
#include "Logging/LogMacros.h"

TRACEINSIGHTSCORE_API DECLARE_LOG_CATEGORY_EXTERN(LogInsights, Verbose, All);

LLM_DECLARE_TAG_API(Insights, TRACEINSIGHTSCORE_API);
