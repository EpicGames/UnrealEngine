// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"

#define UE_API TRACEINSIGHTSCORE_API

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class EDialogResponse
{
	OK = 0,
	Cancel = 1
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMessageDialogUtils
{
public:
	static UE_API EDialogResponse ShowChoiceDialog(const FText& Title, const FText& Content);
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef UE_API
