// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Input/Events.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Geometry.h"
#include "Styling/SlateColor.h"

#include "Templates/SharedPointer.h"

#define UE_API TRACEINSIGHTSCORE_API

namespace UE::Insights
{

class FMiscUtils
{
public:
	/**
	 * Starts a new Unreal Insights instance.
	 * @param InCmdLine - The command line passed to the new UnrealInsights.exe process
	 * @returns true if UnrealInsights process is created successfully.
	 */
	static UE_API bool OpenUnrealInsights(const TCHAR* InCmdLine = nullptr);
};

class FVersionWidget : public TSharedFromThis<FVersionWidget>
{
public:
	enum class EDisplayMode
	{
		Version,
		VersionAndMem,
		VersionAndDetailedMem,

		Count
	};

public:
	FVersionWidget() {}
	virtual ~FVersionWidget() = default;

	UE_API virtual FText GetWidgetText() const;
	UE_API virtual FSlateColor GetColor() const;
	UE_API virtual FReply OnDoubleClicked(const FGeometry&, const FPointerEvent&);
	UE_API virtual TSharedRef<SWidget> CreateWidget();

	EDisplayMode GetDisplayMode() const
	{
		return DisplayMode;
	}
	void SetDisplayMode(EDisplayMode InDisplayMode)
	{
		DisplayMode = InDisplayMode;
	}
	void NextDisplayMode()
	{
		DisplayMode = EDisplayMode((int32(DisplayMode) + 1) % int32(EDisplayMode::Count));
	}

private:
	EDisplayMode DisplayMode = EDisplayMode::Version;
};

} // namespace UE::Insights

#undef UE_API
