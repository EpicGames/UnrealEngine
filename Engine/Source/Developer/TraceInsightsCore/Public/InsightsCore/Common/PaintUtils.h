// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Rendering/DrawElements.h"

#include "InsightsCore/Config.h"

#define MAKE_PAINT_GEOMETRY_PT(Geometry, X, Y)       Geometry.ToPaintGeometry(FSlateLayoutTransform(1.0f, FVector2D(X, Y)))
#define MAKE_PAINT_GEOMETRY_RC(Geometry, X, Y, W, H) Geometry.ToPaintGeometry(FVector2D(W, H), FSlateLayoutTransform(1.0f, FVector2D(X, Y)))

struct FGeometry;
class FSlateRect;
class WidgetStyle;
enum class ESlateDrawEffect : uint8;
class FSlateWindowElementList;

#if UE_INSIGHTS_CORE_BACKWARD_COMPATIBILITY_UE54
struct FDrawContext;
#endif // UE_INSIGHTS_CORE_BACKWARD_COMPATIBILITY_UE54

namespace UE::Insights
{

/**
 * Holds current state provided by OnPaint function, used to simplify drawing.
 */
class FDrawContext
{
public:
	FDrawContext(
			const FGeometry& InGeometry,
			const FSlateRect& InCullingRect,
			const FWidgetStyle& InWidgetStyle,
			const ESlateDrawEffect InDrawEffects,
			FSlateWindowElementList& InOutElementList,
			int32& InOutLayerId)
		: Geometry(InGeometry)
		, CullingRect(InCullingRect)
		, WidgetStyle(InWidgetStyle)
		, DrawEffects(InDrawEffects)
		, ElementList(InOutElementList)
		, LayerId(InOutLayerId)
	{
	}

	/**
	 * Non-copyable
	 */
	FDrawContext(const FDrawContext&) = delete;
	FDrawContext& operator=(const FDrawContext&) = delete;

#if UE_INSIGHTS_CORE_BACKWARD_COMPATIBILITY_UE54
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	using FDrawContextOld = ::FDrawContext;
	inline explicit FDrawContext(FDrawContextOld&);
	inline FDrawContext& operator=(FDrawContextOld&);
	inline operator FDrawContextOld& ();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // UE_INSIGHTS_CORE_BACKWARD_COMPATIBILITY_UE54

	inline void DrawBox(const float X, const float Y, const float W, const float H, const FSlateBrush* Brush, const FLinearColor& Color) const
	{
		FSlateDrawElement::MakeBox(ElementList, LayerId, MAKE_PAINT_GEOMETRY_RC(Geometry, X, Y, W, H), Brush, DrawEffects, Color);
	}

	inline void DrawBox(const int32 InLayer, const float X, const float Y, const float W, const float H, const FSlateBrush* Brush, const FLinearColor& Color) const
	{
		FSlateDrawElement::MakeBox(ElementList, InLayer, MAKE_PAINT_GEOMETRY_RC(Geometry, X, Y, W, H), Brush, DrawEffects, Color);
	}

	inline void DrawRotatedBox(const float X, const float Y, const float W, const float H, const FSlateBrush* Brush, const FLinearColor& Color, float Angle, TOptional<FVector2D> RotationPoint) const
	{
		FSlateDrawElement::MakeRotatedBox(ElementList, LayerId, MAKE_PAINT_GEOMETRY_RC(Geometry, X, Y, W, H), Brush, DrawEffects, Angle, RotationPoint, FSlateDrawElement::RelativeToElement, Color);
	}

	inline void DrawText(const float X, const float Y, const FString& Text, const FSlateFontInfo& Font, const FLinearColor& Color) const
	{
		FSlateDrawElement::MakeText(ElementList, LayerId, MAKE_PAINT_GEOMETRY_PT(Geometry, X, Y), Text, Font, DrawEffects, Color);
	}

	inline void DrawText(const int32 InLayer, const float X, const float Y, const FString& Text, const FSlateFontInfo& Font, const FLinearColor& Color) const
	{
		FSlateDrawElement::MakeText(ElementList, InLayer, MAKE_PAINT_GEOMETRY_PT(Geometry, X, Y), Text, Font, DrawEffects, Color);
	}

	inline void DrawText(const float X, const float Y, const FString& Text, const int32 StartIndex, const int32 EndIndex, const FSlateFontInfo& Font, const FLinearColor& Color) const
	{
		FSlateDrawElement::MakeText(ElementList, LayerId, MAKE_PAINT_GEOMETRY_PT(Geometry, X, Y), Text, StartIndex, EndIndex, Font, DrawEffects, Color);
	}

	inline void DrawText(const int32 InLayer, const float X, const float Y, const FString& Text, const int32 StartIndex, const int32 EndIndex, const FSlateFontInfo& Font, const FLinearColor& Color) const
	{
		FSlateDrawElement::MakeText(ElementList, InLayer, MAKE_PAINT_GEOMETRY_PT(Geometry, X, Y), Text, StartIndex, EndIndex, Font, DrawEffects, Color);
	}

	inline void DrawTextAligned(EHorizontalAlignment HAlign, const float X, const float Y, const FString& Text, const FSlateFontInfo& Font, const FLinearColor& Color) const
	{
		float TextX = X;
		if (HAlign == HAlign_Right || HAlign == HAlign_Center)
		{
			const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
			const float FontScale = Geometry.Scale;
			const float TextWidth = static_cast<float>(FontMeasureService->Measure(Text, Font, FontScale).X / FontScale);
			if (HAlign == HAlign_Right)
			{
				TextX -= TextWidth;
			}
			else
			{
				TextX -= TextWidth / 2;
			}
		}
		FSlateDrawElement::MakeText(ElementList, LayerId, MAKE_PAINT_GEOMETRY_PT(Geometry, TextX, Y), Text, Font, DrawEffects, Color);
	}

	inline void DrawSpline(uint32 InLayer, const float X, const float Y, const FVector2D& InStart, const FVector2D& InStartDir, const FVector2D& InEnd, const FVector2D& InEndDir, float InThickness = 0.0f, const FLinearColor& InTint=FLinearColor::White) const
	{
		FSlateDrawElement::MakeSpline(ElementList, InLayer, MAKE_PAINT_GEOMETRY_PT(Geometry, X, Y), InStart, InStartDir, InEnd, InEndDir, InThickness, DrawEffects, InTint);
	}

	inline void DrawLines(uint32 InLayer, const float X, const float Y, const TArray<FVector2D>& Points, const FLinearColor& InTint = FLinearColor::White, bool bAntialias = true, float Thickness = 1.0f) const
	{
		FSlateDrawElement::MakeLines(ElementList, InLayer, MAKE_PAINT_GEOMETRY_PT(Geometry, X, Y), Points, DrawEffects, InTint, bAntialias, Thickness);
	}

public:
	// Accessors
	const FGeometry& Geometry;
	const FSlateRect& CullingRect;
	const FWidgetStyle& WidgetStyle;
	const ESlateDrawEffect DrawEffects;

	FSlateWindowElementList& ElementList;
	int32& LayerId;
};

} // namespace UE::Insights

#if UE_INSIGHTS_CORE_BACKWARD_COMPATIBILITY_UE54

//struct UE_DEPRECATED(5.5, "FDrawContext struct was moved inside UE::Insights namespace") FDrawContext;
struct FDrawContext final : public UE::Insights::FDrawContext
{
	FDrawContext(
			const FGeometry& InGeometry,
			const FSlateRect& InCullingRect,
			const FWidgetStyle& InWidgetStyle,
			const ESlateDrawEffect InDrawEffects,
			FSlateWindowElementList& InOutElementList,
			int32& InOutLayerId)
		: UE::Insights::FDrawContext(InGeometry, InCullingRect, InWidgetStyle, InDrawEffects, InOutElementList, InOutLayerId)
	{
	}

	/**
	 * Non-copyable
	 */
	FDrawContext(const FDrawContext&) = delete;
	FDrawContext& operator=(const FDrawContext&) = delete;

	explicit FDrawContext(UE::Insights::FDrawContext& Other)
		: UE::Insights::FDrawContext(Other.Geometry, Other.CullingRect, Other.WidgetStyle, Other.DrawEffects, Other.ElementList, Other.LayerId)
	{
	}

	FDrawContext& operator=(UE::Insights::FDrawContext& Other)
	{
		new (this) FDrawContext(Other.Geometry, Other.CullingRect, Other.WidgetStyle, Other.DrawEffects, Other.ElementList, Other.LayerId);
		return *this;
	}

	FDrawContext& operator()(UE::Insights::FDrawContext& Other)
	{
		return *reinterpret_cast<FDrawContext*>(&Other);
	}

	inline void DrawLines(uint32 InLayer, const float X, const float Y, const TArray<FVector2D>& Points, ESlateDrawEffect InDrawEffects = ESlateDrawEffect::None, const FLinearColor& InTint = FLinearColor::White, bool bAntialias = true, float Thickness = 1.0f) const
	{
		FSlateDrawElement::MakeLines(ElementList, InLayer, MAKE_PAINT_GEOMETRY_PT(Geometry, X, Y), Points, InDrawEffects, InTint, bAntialias, Thickness);
	}
};

namespace UE::Insights
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS

FDrawContext::FDrawContext(FDrawContextOld& Other)
	: FDrawContext(Other.Geometry, Other.CullingRect, Other.WidgetStyle, Other.DrawEffects, Other.ElementList, Other.LayerId)
{
}

FDrawContext& FDrawContext::operator=(FDrawContextOld& Other)
{
	new (this) FDrawContext(Other.Geometry, Other.CullingRect, Other.WidgetStyle, Other.DrawEffects, Other.ElementList, Other.LayerId);
	return *this;
}

FDrawContext::operator FDrawContextOld& ()
{
	return *reinterpret_cast<FDrawContextOld*>(this);
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS
} // namespace UE::Insights

#endif // UE_INSIGHTS_CORE_BACKWARD_COMPATIBILITY_UE54
