// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Simple RTTI implementation.

// Note: The SimpleRTTI class hierarchy may be different than the actual class hierarchy.
//       Ex.: Some classes in the actual hierarchy can be hidden (do not use INSIGHTS_DECLARE_RTTI, so just inherits type info from parent class).

// Macro to be used in declaration of the base class of the SimpleRTTI class hierarchy.
#define INSIGHTS_DECLARE_RTTI_BASE(BaseClassName, ...) \
	public: \
		static const FName& GetStaticTypeName() { return BaseClassName::TypeName; } \
		virtual const FName& GetTypeName() const { return BaseClassName::GetStaticTypeName(); } \
		virtual bool IsKindOf(const FName& InTypeName) const { return InTypeName == BaseClassName::GetStaticTypeName(); } \
		template<typename Type> bool Is() const { return IsKindOf(Type::GetStaticTypeName()); } \
		template<typename Type> const Type& As() const { return *static_cast<const Type*>(this); } \
		template<typename Type> Type& As() { return *static_cast<Type*>(this); } \
	private: \
		static __VA_ARGS__ const FName TypeName;

// Macro to be used in declaration of a derived class in a SimpleRTTI hierarchy.
#define INSIGHTS_DECLARE_RTTI(ClassName, BaseClassName, ...) \
	public: \
		static const FName& GetStaticTypeName() { return ClassName::TypeName; } \
		virtual const FName& GetTypeName() const override { return ClassName::GetStaticTypeName(); } \
		virtual bool IsKindOf(const FName& InTypeName) const override { return InTypeName == ClassName::GetStaticTypeName() || BaseClassName::IsKindOf(InTypeName); } \
	private: \
		static __VA_ARGS__ const FName TypeName;

// Macro to be used in implementation of a class in a SimpleRTTI hierarchy.
#define INSIGHTS_IMPLEMENT_RTTI(ClassName) \
	const FName ClassName::TypeName = TEXT(#ClassName);
