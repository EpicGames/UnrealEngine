// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "HAL/PlatformTime.h"

namespace UE::Insights
{

// Simple stopwatch.
struct FStopwatch
{
	uint64 AccumulatedTime = 0;
	uint64 StartTime = 0;
	bool bIsStarted = false;

	void Start()
	{
		if (!bIsStarted)
		{
			bIsStarted = true;
			StartTime = FPlatformTime::Cycles64();
		}
	}

	void Stop()
	{
		if (bIsStarted)
		{
			bIsStarted = false;
			AccumulatedTime += FPlatformTime::Cycles64() - StartTime;
		}
	}

	void Update()
	{
		if (bIsStarted)
		{
			uint64 CrtTime = FPlatformTime::Cycles64();
			AccumulatedTime += CrtTime - StartTime;
			StartTime = CrtTime;
		}
	}

	void Restart()
	{
		AccumulatedTime = 0;
		bIsStarted = true;
		StartTime = FPlatformTime::Cycles64();
	}

	void Reset()
	{
		AccumulatedTime = 0;
		StartTime = 0;
		bIsStarted = false;
	}

	double GetAccumulatedTime() const
	{
		return FStopwatch::Cycles64ToSeconds(AccumulatedTime);
	}

	uint64 GetAccumulatedTimeMs() const
	{
		return FStopwatch::Cycles64ToMilliseconds(AccumulatedTime);
	}

	static double Cycles64ToSeconds(const uint64 Cycles64)
	{
		return static_cast<double>(Cycles64) * FPlatformTime::GetSecondsPerCycle64();
	}

	static uint64 Cycles64ToMilliseconds(const uint64 Cycles64)
	{
		const double Milliseconds = FMath::RoundToDouble(static_cast<double>(Cycles64 * 1000) * FPlatformTime::GetSecondsPerCycle64());
		return static_cast<uint64>(Milliseconds);
	}
};

} // namespace UE::Insights
