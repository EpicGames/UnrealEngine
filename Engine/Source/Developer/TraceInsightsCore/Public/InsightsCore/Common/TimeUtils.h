// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"

#include "InsightsCore/Config.h"

/*
double is a 64 bit IEEE 754 double precision Floating Point Number
1 bit for the sign, 11 bits for the exponent, and 52* bits for the value
15.5 decimal digits of precision (max value*: 4`503`599`627`370`495)
|------------------------------------------------------------------------------
| 900`000.000`000`001     --> up to 10 days   with    1 ns precision
|  90`000.000`000`000`1   --> up to 25 hours  with  0.1 ns precision (100 ps)
|   9`000.000`000`000`01  --> up to 2.5 hours with 0.01 ns precision (10 ps)
|     900.000`000`000`001 --> up to 15 min    with    1 ps precision
|------------------------------------------------------------------------------
|   3`600.000`000`000`001 --> 1 ps precision at 1 hour : NOT OK (16 digits)!!!
|  86`400.000`000`000`001 --> 1 ps precision at 1 day  : NOT OK (17 digits)!!!
|------------------------------------------------------------------------------
*/

namespace UE::Insights
{

struct FTimeValue
{
	static constexpr double Picosecond = 0.000000000001;
	static constexpr double Nanosecond = 0.000000001;
	static constexpr double Microsecond = 0.000001;
	static constexpr double Millisecond = 0.001;
	static constexpr double Second = 1.0;
	static constexpr double Minute = 60.0;
	static constexpr double Hour = 3600.0;
	static constexpr double Day = 86400.0;
	static constexpr double Week = 604800.0;

	double Value;
};

struct FTimeSplit
{
	union
	{
		int32 Units[8]; // Units[0] = Days, Units[1] = Hours, ..., Units[7] = Picoseconds
		struct
		{
			int32 Days;
			int32 Hours;
			int32 Minutes;
			int32 Seconds;
			int32 Milliseconds;
			int32 Microseconds;
			int32 Nanoseconds;
			int32 Picoseconds;
		};
	};
	bool bIsZero;
	bool bIsNegative;
	bool bIsInfinite;
	bool bIsNaN;
};

TRACEINSIGHTSCORE_API FString FormatTimeValue(const double Duration, const int32 NumDigits = 2);
TRACEINSIGHTSCORE_API FString FormatTimeAuto(const double Duration, const int32 NumDigits = 2);
TRACEINSIGHTSCORE_API FString FormatTimeMs(const double Duration, const int32 NumDigits = 3, bool bAddTimeUnit = false);
TRACEINSIGHTSCORE_API FString FormatTime(const double Time, const double Precision = 0.0);
TRACEINSIGHTSCORE_API FString FormatTimeHMS(const double Time, const double Precision = 0.0);
TRACEINSIGHTSCORE_API void SplitTime(const double Time, FTimeSplit& OutTimeSplit);
TRACEINSIGHTSCORE_API FString FormatTimeSplit(const FTimeSplit& TimeSplit, const double Precision = 0.0);
TRACEINSIGHTSCORE_API FString FormatTimeSplit(const double Time, const double Precision = 0.0);

void TestTimeFormatting();
void TestTimeAutoFormatting();
void TestOptimizationIssue();

} // namespace UE::Insights
