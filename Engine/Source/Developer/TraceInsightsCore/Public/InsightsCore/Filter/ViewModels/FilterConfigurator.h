// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "InsightsCore/Filter/ViewModels/FilterConfiguratorNode.h"
#include "InsightsCore/Filter/ViewModels/Filters.h"
#include "InsightsCore/Filter/ViewModels/IFilterExecutor.h"

#define UE_API TRACEINSIGHTSCORE_API

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FFilterConfigurator : public IFilterExecutor
{
public:
	UE_API FFilterConfigurator();

	UE_API FFilterConfigurator(const FFilterConfigurator& Other);
	UE_API FFilterConfigurator& operator=(const FFilterConfigurator& Other);

	UE_API bool operator==(const FFilterConfigurator& Other) const;
	bool operator!=(const FFilterConfigurator& Other) const { return !(*this == Other); }

	UE_API virtual ~FFilterConfigurator();

	bool IsEmpty() const { return RootNode->GetChildrenCount() == 0; }

	/** Called to update the internal state of some filters */
	void Update() { RootNode->Update(); }

	UE_API virtual bool ApplyFilters(const FBaseFilterContext& Context) const override;

	UE_API bool IsKeyUsed(int32 Key) const;

	FFilterConfiguratorNodePtr GetRootNode() { return RootNode; }
	TSharedPtr<TArray<TSharedPtr<FFilter>>>& GetAvailableFilters() { return AvailableFilters; }

	void Add(TSharedPtr<FFilter> InFilter) { AvailableFilters->Add(InFilter); }

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// OnDestroyedEvent

public:
	/** The event to execute when an instance is destroyed. */
	DECLARE_MULTICAST_DELEGATE(FOnDestroyedEvent);
	FOnDestroyedEvent& GetOnDestroyedEvent() { return OnDestroyedEvent; }

private:
	FOnDestroyedEvent OnDestroyedEvent;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// OnChangesCommittedEvent

public:
	/** The event to execute when the changes to the Filter Widget are saved by clicking on the OK Button. */
	DECLARE_MULTICAST_DELEGATE(FOnChangesCommittedEvent);
	FOnChangesCommittedEvent& GetOnChangesCommittedEvent() { return OnChangesCommittedEvent; }

private:
	FOnChangesCommittedEvent OnChangesCommittedEvent;

	////////////////////////////////////////////////////////////////////////////////////////////////////

private:
	UE_API void ComputeUsedKeys();

private:
	FFilterConfiguratorNodePtr RootNode;

	TSharedPtr<TArray<TSharedPtr<FFilter>>> AvailableFilters;

	TSet<int32> KeysUsed;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef UE_API
