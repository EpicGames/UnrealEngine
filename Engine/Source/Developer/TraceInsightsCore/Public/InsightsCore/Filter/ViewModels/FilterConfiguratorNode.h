// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "InsightsCore/Filter/ViewModels/Filters.h"
#include "InsightsCore/Table/ViewModels/BaseTreeNode.h"

#define UE_API TRACEINSIGHTSCORE_API

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Type definition for shared pointers to instances of FFilterConfiguratorNode. */
typedef TSharedPtr<class FFilterConfiguratorNode> FFilterConfiguratorNodePtr;

/** Type definition for shared references to instances of FFilterConfiguratorNode. */
typedef TSharedRef<class FFilterConfiguratorNode> FFilterConfiguratorRef;

/** Type definition for shared references to const instances of FFilterConfiguratorNode. */
typedef TSharedRef<const class FFilterConfiguratorNode> FFilterConfiguratorRefConst;

/** Type definition for weak references to instances of FFilterConfiguratorNode. */
typedef TWeakPtr<class FFilterConfiguratorNode> FFilterConfiguratorNodeWeak;

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Class used to store information about a filter node
 */
class FFilterConfiguratorNode : public FBaseTreeNode
{
	INSIGHTS_DECLARE_RTTI(FFilterConfiguratorNode, FBaseTreeNode, UE_API)

public:
	/** Initialization constructor for the filter configurator node. */
	UE_API FFilterConfiguratorNode(bool bInIsGroup);

	UE_DEPRECATED(5.8, "Use constructor without Name")
	UE_API FFilterConfiguratorNode(FName InName, bool bInIsGroup) : FFilterConfiguratorNode(bInIsGroup) {}

	static UE_API TSharedRef<FFilterConfiguratorNode> DeepCopy(const FFilterConfiguratorNode& Node);

	UE_API bool operator==(const FFilterConfiguratorNode& Other) const;

	virtual ~FFilterConfiguratorNode() = default;

	UE_API void SetAvailableFilters(TSharedPtr<TArray<TSharedPtr<FFilter>>> InAvailableFilters);
	TSharedPtr<TArray<TSharedPtr<FFilter>>> GetAvailableFilters() { return AvailableFilters; }

	UE_API void SetSelectedFilter(TSharedPtr<FFilter> InSelectedFilter);
	TSharedPtr<FFilter> GetSelectedFilter() const { return SelectedFilter; }

	UE_API void SetSelectedFilterOperator(TSharedPtr<IFilterOperator> InSelectedFilterOperator);
	UE_API TSharedPtr<const IFilterOperator> GetSelectedFilterOperator() const;

	UE_API const TArray<TSharedPtr<FFilterGroupOperator>>& GetFilterGroupOperators() const;
	void SetSelectedFilterGroupOperator(TSharedPtr<FFilterGroupOperator> InSelectedFilterGroupOperator) { SelectedFilterGroupOperator = InSelectedFilterGroupOperator; }
	TSharedPtr<FFilterGroupOperator> GetSelectedFilterGroupOperator() const { return SelectedFilterGroupOperator; }

	TSharedPtr<TArray<TSharedPtr<IFilterOperator>>> GetAvailableFilterOperators() const { return AvailableFilterOperators; }

	const FString& GetTextBoxValue() { return TextBoxValue; }
	UE_API void SetTextBoxValue(const FString& InValue);

	UE_API bool ApplyFilters(const FBaseFilterContext& Context) const;

	UE_API void GetUsedKeys(TSet<int32>& GetUsedKeys) const;

	UE_API void ProcessFilter();

	UE_API void Update();

	TSharedPtr<FFilterState> GetSelectedFilterState() { return FilterState; }

private:
	UE_API FFilterConfiguratorNode& operator=(const FFilterConfiguratorNode& Other);

	TSharedPtr<TArray<TSharedPtr<FFilter>>> AvailableFilters;

	TSharedPtr<FFilter> SelectedFilter;

	TSharedPtr<FFilterGroupOperator> SelectedFilterGroupOperator;

	TSharedPtr<TArray<TSharedPtr<IFilterOperator>>> AvailableFilterOperators;

	FString TextBoxValue;

	TSharedPtr<FFilterState> FilterState;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef UE_API
