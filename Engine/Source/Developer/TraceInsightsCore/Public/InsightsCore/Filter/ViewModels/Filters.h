// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"
#include "Misc/TVariant.h"
#include "Templates/SharedPointer.h"

#include "InsightsCore/Common/SimpleRtti.h"

#define UE_API TRACEINSIGHTSCORE_API

class FSpawnTabArgs;
class SDockTab;
class SHorizontalBox;
class SWidget;

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class EFilterDataType : uint8
{
	Void = 0, // not set

	//Bool,
	//Uint32,
	//Uint64,
	//Int32,
	Int64,
	//Float,
	Double,
	//StringView,
	//AnsiStringView,
	//Utf8StringView,
	//CString,
	//AnsiCString,
	//Utf8CString,
	String,
	//AnsiString,
	//Utf8String,
	//ByteArray,
	Custom, // For complex filters that are implemented as separate classes.
	StringInt64Pair, // Displayed as a string but translates to a Int64 key.

	Count
};

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class EFilterOperator : uint8
{
	Eq = 0, // Equals
	NotEq = 1, // Not Equals
	Lt = 2, // Less Than
	Lte = 3, // Less than or equal to
	Gt = 4, // Greater than
	Gte = 5, // Greater than or equal to
	Contains = 6,
	NotContains = 7,
};

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class EFilterField : int32
{
	StartTime = 0,
	EndTime = 1,
	Duration = 2,
	TrackName = 3,
	TimerId = 4,
	TimerName = 5,
	CoreEventName = 6,
	RegionName = 7,
	RegionCategory = 8,
	Metadata = 9,
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class IFilterOperator
{
public:
	virtual EFilterOperator GetKey() const = 0;
	virtual FString GetName() const = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename T>
class FFilterOperator : public IFilterOperator
{
public:
	typedef TFunction<bool(T, T)> FOperatorFunc;

public:
	FFilterOperator(EFilterOperator InKey, FString InName, FOperatorFunc InFunc)
		: Key(InKey)
		, Name(InName)
		, Func(InFunc)
	{
	}

	virtual ~FFilterOperator() = default;

	virtual EFilterOperator GetKey() const override { return Key; }
	virtual FString GetName() const override { return Name; };

	bool Apply(T InValueA, T InValueB) const { return Func(InValueA, InValueB); }

private:
	EFilterOperator Key;
	FString Name;
	FOperatorFunc Func;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class EFilterGroupOperator
{
	And = 0,
	Or = 1,
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FFilterGroupOperator
{
public:
	FFilterGroupOperator(EFilterGroupOperator InType, FText InName, FText InDesc)
		: Type(InType)
		, Name(InName)
		, Desc(InDesc)
	{
	}

	EFilterGroupOperator GetType() const { return Type; }
	const FText& GetName() const { return Name; }
	const FText& GetDesc() const { return Desc; }

private:
	EFilterGroupOperator Type;
	FText Name;
	FText Desc;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class IFilterValueConverter
{
public:
	virtual bool Convert(const FString& Input, int64& Output, FText& OutError) const { unimplemented(); return false; }
	virtual bool Convert(const FString& Input, double& Output, FText& OutError) const { unimplemented(); return false; }
	virtual FText GetTooltipText() const { return FText(); }
	virtual FText GetHintText() const { return FText(); }
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FBaseFilterContext
{
public:
	FBaseFilterContext() = default;
	virtual ~FBaseFilterContext() = default;

	virtual bool GetDefaultReturnValue() const
	{
		return false;
	}

	virtual bool HasData(int32 Key) const = 0;
	virtual EFilterDataType GetDataType(int32 Key) const = 0;

	//virtual bool GetDataAsBool(int32 Key) const                                 { return false; }
	//virtual uint32 GetDataAsUint32(int32 Key) const                             { return 0; }
	//virtual uint64 GetDataAsUint64(int32 Key) const                             { return 0; }
	//virtual int32 GetDataAsInt32(int32 Key) const                               { return 0; }
	virtual int64 GetDataAsInt64(int32 Key) const                               { return 0; }
	//virtual float GetDataAsFloat(int32 Key) const                               { return 0.0f; }
	virtual double GetDataAsDouble(int32 Key) const                             { return 0.0; }
	virtual FStringView GetDataAsStringView(int32 Key) const                    { return GetDataAsString(Key); }
	//virtual FAnsiStringView GetDataAsAnsiStringView(int32 Key) const            { return FAnsiStringView(); }
	//virtual FUtf8StringView GetDataAsUtf8StringView(int32 Key) const            { return FUtf8StringView();}
	virtual const TCHAR* GetDataAsCString(int32 Key) const                      { return *GetDataAsString(Key); }
	//virtual const ANSICHAR* GetDataAsAnsiCString(int32 Key) const               { return nullptr; }
	//virtual const UTF8CHAR* GetDataAsUtf8CString(int32 Key) const               { return nullptr; }
	virtual const FString& GetDataAsString(int32 Key) const                     { return EmptyString; }
	//virtual const FAnsiString& GetDataAsAnsiString(int32 Key) const             { return EmptyAnsiString; }
	//virtual const FUtf8String& GetDataAsUtf8String(int32 Key) const             { return EmptyUtf8String; }
	//virtual TTuple<const uint8*, uint64> GetDataAsByteArray(int32 Key) const    { return TTuple<const uint8*, uint64>(nullptr, 0); }
	//virtual const void* GetDataAsPointer(int32 Key) const                       { return nullptr; }

protected:
	UE_API static FString EmptyString;
	//static FAnsiString EmptyAnsiString;
	//static FUtf8String EmptyUtf8String;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FFilterContext : public FBaseFilterContext
{
public:
	typedef TVariant<int64, double, FString> ContextData;

public:
	FFilterContext() = default;
	virtual ~FFilterContext() = default;

	//////////////////////////////////////////////////

	UE_DEPRECATED(5.8, "Use GetDefaultReturnValue() instead")
	bool GetReturnValueForUnsetFilters() const { return bDefaultReturnValue; }

	UE_DEPRECATED(5.8, "Use custom class instead")
	void SetReturnValueForUnsetFilters(bool InValue) { bDefaultReturnValue = InValue; }

	virtual bool GetDefaultReturnValue() const override { return bDefaultReturnValue; }

	UE_DEPRECATED(5.8, "Use HasData() instead")
	bool HasFilterData(int32 Key) const
	{
		return DataMap.Contains(Key);
	}

	virtual bool HasData(int32 Key) const override
	{
		return DataMap.Contains(Key);
	}

	virtual EFilterDataType GetDataType(int32 Key) const override
	{
		const ContextData* Data = DataMap.Find(Key);
		if (Data)
		{
			if (Data->IsType<int64>())
			{
				return EFilterDataType::Int64;
			}
			if (Data->IsType<double>())
			{
				return EFilterDataType::Double;
			}
			if (Data->IsType<FString>())
			{
				return EFilterDataType::String;
			}
		}
		return EFilterDataType::Void;
	}

	//////////////////////////////////////////////////

	template<typename T>
	void AddFilterData(int32 Key, const T& InData)
	{
		ContextData VariantData;
		VariantData.Set<T>(InData);
		DataMap.Add(Key, VariantData);
	}

	template<typename T>
	void SetFilterData(int32 Key, const T& InData)
	{
		DataMap[Key].Set<T>(InData);
	}

	template<typename T>
	void GetFilterData(int32 Key, T& OutData) const
	{
		const ContextData* Data = DataMap.Find(Key);
		check(Data && Data->IsType<T>());
		OutData = Data->Get<T>();
	}

	UE_DEPRECATED(5.8, "Use GetDataAsString instead")
	const FString& GetFilterDataAsString(int32 Key) const
	{
		const ContextData* Data = DataMap.Find(Key);
		check(Data && Data->IsType<FString>());
		return Data->Get<FString>();
	}

	//////////////////////////////////////////////////

	virtual int64 GetDataAsInt64(int32 Key) const override
	{
		const ContextData* Data = DataMap.Find(Key);
		check(Data && Data->IsType<int64>())
		return Data->Get<int64>();
	}
	virtual double GetDataAsDouble(int32 Key) const override
	{
		const ContextData* Data = DataMap.Find(Key);
		check(Data && Data->IsType<double>());
		return Data->Get<double>();
	}
	virtual const FString& GetDataAsString(int32 Key) const override
	{
		const ContextData* Data = DataMap.Find(Key);
		check(Data && Data->IsType<FString>());
		return Data->Get<FString>();
	}

	//////////////////////////////////////////////////

private:
	TMap<int32, ContextData> DataMap;
	bool bDefaultReturnValue = true;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

typedef TSharedPtr<const TArray<TSharedPtr<IFilterOperator>>> SupportedOperatorsArrayConstPtr;
typedef TSharedPtr<TArray<TSharedPtr<IFilterOperator>>> SupportedOperatorsArrayPtr;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FFilter;

class FFilterState
{
	INSIGHTS_DECLARE_RTTI_BASE(FFilterState, UE_API)

public:
	FFilterState(TSharedRef<FFilter> InFilter)
		: Filter(InFilter)
	{
	}

	virtual ~FFilterState() = default;

	void SetSelectedOperator(TSharedPtr<IFilterOperator> InOperator) { SelectedOperator = InOperator; }
	TSharedPtr<const IFilterOperator> GetSelectedOperator() const { return SelectedOperator; }

	virtual void Update() {}

	UE_API virtual bool ApplyFilter(const FBaseFilterContext& Context) const;

	UE_API virtual void SetFilterValue(FString InTextValue);

	virtual bool HasCustomUI() const { return false; }
	virtual void AddCustomUI(TSharedRef<SHorizontalBox> LeftBox) {}

	UE_API virtual bool Equals(const FFilterState& Other) const;
	UE_API virtual TSharedRef<FFilterState> DeepCopy() const;

protected:
	TSharedRef<FFilter> Filter;
	TSharedPtr<IFilterOperator> SelectedOperator;
	FFilterContext::ContextData FilterValue;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FFilter : public TSharedFromThis<FFilter>
{
	INSIGHTS_DECLARE_RTTI_BASE(FFilter, UE_API)

public:
	FFilter(int32 InKey, FText InName, FText InDesc, EFilterDataType InDataType, TSharedPtr<IFilterValueConverter> InConverter, SupportedOperatorsArrayPtr InSupportedOperators)
		: Key(InKey)
		, Name(InName)
		, Desc(InDesc)
		, DataType(InDataType)
		, Converter(InConverter)
		, SupportedOperators(InSupportedOperators)
	{
	}

	virtual ~FFilter() = default;

	int32 GetKey() const { return Key; }
	const FText& GetName() const { return Name; }
	const FText& GetDesc() const { return Desc; }
	EFilterDataType GetDataType() const { return DataType; }
	const TSharedPtr<IFilterValueConverter>& GetConverter() const { return Converter; }
	SupportedOperatorsArrayConstPtr GetSupportedOperators() const { return SupportedOperators; }

	virtual TSharedRef<FFilterState> BuildFilterState() { return MakeShared<FFilterState>(SharedThis(this)); }
	virtual TSharedRef<FFilterState> BuildFilterState(const FFilterState& Other) { return MakeShared<FFilterState>(Other); }

protected:
	int32 Key;
	FText Name;
	FText Desc;
	EFilterDataType DataType;
	TSharedPtr<IFilterValueConverter> Converter;
	SupportedOperatorsArrayPtr SupportedOperators;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FFilterWithSuggestionsValueConverter : public IFilterValueConverter
{
public:
	virtual bool Convert(const FString& Input, double& Output, FText& OutError) const override { return true; }
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual FText GetHintText() const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FFilterWithSuggestions : public FFilter
{
	INSIGHTS_DECLARE_RTTI(FFilterWithSuggestions, FFilter, UE_API)

public:
	typedef TFunction<void(const FString& /*Text*/, TArray<FString>& OutSuggestions)> FGetSuggestionsCallback;

public:
	FFilterWithSuggestions(int32 InKey, FText InName, FText InDesc, EFilterDataType InDataType, TSharedPtr<IFilterValueConverter> InConverter, SupportedOperatorsArrayPtr InSupportedOperators)
		: FFilter(InKey, InName, InDesc, InDataType, InConverter, InSupportedOperators)
	{
		if (!InConverter.IsValid())
		{
			// Add a default converter to add a hint text describing that this filter supports auto-complete.
			Converter = MakeShared<FFilterWithSuggestionsValueConverter>();
		}
	}

	virtual ~FFilterWithSuggestions() = default;

	const FGetSuggestionsCallback& GetCallback() const { return Callback; }
	void SetCallback(FGetSuggestionsCallback InCallback) { Callback = InCallback; }
	void GetSuggestions(const FString& Text, TArray<FString>& OutSuggestions) { Callback(Text, OutSuggestions); }

private:
	FGetSuggestionsCallback Callback;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FFilterStorage
{
public:
	UE_API FFilterStorage();
	UE_API ~FFilterStorage();

	const TArray<TSharedPtr<FFilterGroupOperator>>& GetFilterGroupOperators() const { return FilterGroupOperators; }

	TSharedPtr<TArray<TSharedPtr<IFilterOperator>>> GetDoubleOperators() const { return DoubleOperators; }
	TSharedPtr<TArray<TSharedPtr<IFilterOperator>>> GetIntegerOperators() const { return IntegerOperators; }
	TSharedPtr<TArray<TSharedPtr<IFilterOperator>>> GetStringOperators() const { return StringOperators; }

private:
	TArray<TSharedPtr<FFilterGroupOperator>> FilterGroupOperators;

	TSharedPtr<TArray<TSharedPtr<IFilterOperator>>> DoubleOperators;
	TSharedPtr<TArray<TSharedPtr<IFilterOperator>>> IntegerOperators;
	TSharedPtr<TArray<TSharedPtr<IFilterOperator>>> StringOperators;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FFilterService
{
public:
	UE_API FFilterService();
	UE_API ~FFilterService();

	static UE_API void Initialize();
	static UE_API void Shutdown();
	static TSharedPtr<FFilterService> Get() { return Instance; }

	UE_API void RegisterTabSpawner();
	UE_API void UnregisterTabSpawner();
	UE_API TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& Args);

	const TArray<TSharedPtr<FFilterGroupOperator>>& GetFilterGroupOperators() const { return FilterStorage.GetFilterGroupOperators(); }

	TSharedPtr<TArray<TSharedPtr<IFilterOperator>>> GetDoubleOperators() const { return FilterStorage.GetDoubleOperators(); }
	TSharedPtr<TArray<TSharedPtr<IFilterOperator>>> GetIntegerOperators() const { return FilterStorage.GetIntegerOperators(); }
	TSharedPtr<TArray<TSharedPtr<IFilterOperator>>> GetStringOperators() const { return FilterStorage.GetStringOperators(); }

	UE_API TSharedPtr<SWidget> CreateFilterConfiguratorWidget(TSharedPtr<class FFilterConfigurator> FilterConfiguratorViewModel, TWeakPtr<SWidget> ParentWidget = nullptr);

private:
	static UE_API const FName FilterConfiguratorTabId;
	static UE_API TSharedPtr<FFilterService> Instance;
	FFilterStorage FilterStorage;
	TSharedPtr<class SAdvancedFilter> PendingWidget;
	TWeakPtr<SWidget> PendingParentWidget;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef UE_API
