// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace UE::Insights
{

class FBaseFilterContext;

class IFilterExecutor
{
public:
	virtual bool ApplyFilters(const FBaseFilterContext& Context) const = 0;
};

} // namespace UE::Insights
