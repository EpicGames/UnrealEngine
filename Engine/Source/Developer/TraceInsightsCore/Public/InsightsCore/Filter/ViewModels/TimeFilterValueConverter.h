// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"

#include "InsightsCore/Filter/ViewModels/Filters.h"

#define UE_API TRACEINSIGHTSCORE_API

namespace UE::Insights
{

class FTimeFilterValueConverter : public IFilterValueConverter
{
public:
	UE_API virtual bool Convert(const FString& Input, double& Output, FText& OutError) const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual FText GetHintText() const override;
};

} // namespace UE::Insights

#undef UE_API
