// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STreeView.h"

#include "InsightsCore/Filter/ViewModels/FilterConfiguratorNode.h"
#include "InsightsCore/Filter/ViewModels/Filters.h"

#define UE_API TRACEINSIGHTSCORE_API

class SDockTab;

namespace UE::Insights
{

class FFilterConfigurator;
class SFilterConfigurator;

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * A custom widget used to configure custom filters.
 */
class SAdvancedFilter: public SCompoundWidget
{
public:
	/** Default constructor. */
	UE_API SAdvancedFilter();

	/** Virtual destructor. */
	UE_API virtual ~SAdvancedFilter();

	SLATE_BEGIN_ARGS(SAdvancedFilter) {}
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 * @param InArgs - The declaration data for this widget
	 */
	UE_API void Construct(const FArguments& InArgs, TSharedPtr<FFilterConfigurator> InFilterConfiguratorViewModel);

	UE_API void Reset();

	void SetParentTab(const TSharedPtr<SDockTab> InTab) { ParentTab = InTab; }
	const TWeakPtr<SDockTab> GetParentTab() { return ParentTab; };

	UE_API void RequestClose();

private:
	UE_API void InitCommandList();

	UE_API FReply OK_OnClicked();

	UE_API FReply Cancel_OnClicked();

private:
	TSharedPtr<SFilterConfigurator> FilterConfigurator;

	TWeakPtr<FFilterConfigurator> OriginalFilterConfiguratorViewModel;

	TSharedPtr<FFilterConfigurator> FilterConfiguratorViewModel;

	TWeakPtr<SDockTab> ParentTab;

	FDelegateHandle OnViewModelDestroyedHandle;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef UE_API
