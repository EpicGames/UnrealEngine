// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STreeView.h"

#include "InsightsCore/Filter/ViewModels/FilterConfiguratorNode.h"
#include "InsightsCore/Filter/ViewModels/Filters.h"

#define UE_API TRACEINSIGHTSCORE_API

class SDockTab;

namespace UE::Insights
{

class FFilterConfigurator;

////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * A custom widget used to configure custom filters.
 */
class SFilterConfigurator: public SCompoundWidget
{
public:
	/** Default constructor. */
	UE_API SFilterConfigurator();

	/** Virtual destructor. */
	UE_API virtual ~SFilterConfigurator();

	SLATE_BEGIN_ARGS(SFilterConfigurator) {}
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 * @param InArgs - The declaration data for this widget
	 */
	UE_API void Construct(const FArguments& InArgs, TSharedPtr<FFilterConfigurator> InFilterConfiguratorViewModel);

	UE_API void Reset();

	/**
	 * Called by STreeView to retrieves the children for the specified parent item.
	 * @param InParent    - The parent node to retrieve the children from.
	 * @param OutChildren - List of children for the parent node.
	 */
	UE_API void TreeView_OnGetChildren(FFilterConfiguratorNodePtr InParent, TArray<FFilterConfiguratorNodePtr>& OutChildren);

	/** Called by STreeView to generate a table row for the specified item. */
	UE_API TSharedRef<class ITableRow> TreeView_OnGenerateRow(FFilterConfiguratorNodePtr TreeNode, const TSharedRef<class STableViewBase>& OwnerTable);

	void SetParentTab(const TSharedPtr<SDockTab> InTab) { ParentTab = InTab; }
	const TWeakPtr<SDockTab> GetParentTab() { return ParentTab; };

	UE_API void RequestClose();

private:
	UE_API void InitCommandList();

	UE_API void SetInitialExpansionRec(const FFilterConfiguratorNodePtr& Node, bool Value);

private:
	TSharedPtr<FFilterConfigurator> FilterConfiguratorViewModel;

	/** The tree widget which holds the filters. */
	TSharedPtr<STreeView<FFilterConfiguratorNodePtr>> TreeView;

	/** Holds the tree view header row widget which display all columns in the tree view. */
	TSharedPtr<class SHeaderRow> TreeViewHeaderRow;

	/** External scrollbar used to synchronize tree view position. */
	TSharedPtr<class SScrollBar> ExternalScrollbar;

	/** A filtered array of group and filter nodes to be displayed in the tree widget. */
	TArray<FFilterConfiguratorNodePtr> GroupNodes;

	TWeakPtr<SDockTab> ParentTab;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef UE_API
