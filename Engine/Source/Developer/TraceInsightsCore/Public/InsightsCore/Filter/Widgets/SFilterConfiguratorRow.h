// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/Input/SComboBox.h"
#include "Widgets/Views/STableRow.h"

#include "InsightsCore/Filter/ViewModels/FilterConfiguratorNode.h"

#define UE_API TRACEINSIGHTSCORE_API

namespace UE::Insights
{

class FFilter;
class FFilterGroupOperator;
class IFilterOperator;

/** Widget that represents a table row in the Filter's tree control. Generates widgets for each column on demand. */
class SFilterConfiguratorRow : public SMultiColumnTableRow<FFilterConfiguratorNodePtr>
{
public:
	SFilterConfiguratorRow() {}
	~SFilterConfiguratorRow() = default;

	SLATE_BEGIN_ARGS(SFilterConfiguratorRow) {}
		SLATE_ARGUMENT(FFilterConfiguratorNodePtr, FilterConfiguratorNodePtr)
	SLATE_END_ARGS()

public:
	UE_API void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

	UE_API virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;

private:
	UE_API const TArray<TSharedPtr<FFilter>>* GetAvailableFilters();

	UE_API TSharedRef<SWidget> AvailableFilters_OnGenerateWidget(TSharedPtr<FFilter> InFilter);

	UE_API void AvailableFilters_OnSelectionChanged(TSharedPtr<FFilter> InFilter, ESelectInfo::Type SelectInfo);

	UE_API FText AvailableFilters_GetSelectionText() const;

	UE_API const TArray<TSharedPtr<IFilterOperator>>* GetAvailableFilterOperators();

	UE_API TSharedRef<SWidget> AvailableFilterOperators_OnGenerateWidget(TSharedPtr<IFilterOperator> InFilter);

	UE_API void AvailableFilterOperators_OnSelectionChanged(TSharedPtr<IFilterOperator> InFilter, ESelectInfo::Type SelectInfo);

	UE_API FText AvailableFilterOperators_GetSelectionText() const;

	UE_API const TArray<TSharedPtr<FFilterGroupOperator>>* GetFilterGroupOperators();

	UE_API TSharedRef<SWidget> FilterGroupOperators_OnGenerateWidget(TSharedPtr<FFilterGroupOperator> InFilter);

	UE_API void FilterGroupOperators_OnSelectionChanged(TSharedPtr<FFilterGroupOperator> InFilter, ESelectInfo::Type SelectInfo);

	UE_API FText FilterGroupOperators_GetSelectionText() const;

	UE_API FReply AddFilter_OnClicked();
	UE_API FReply DeleteFilter_OnClicked();

	UE_API FReply AddGroup_OnClicked();
	UE_API FReply DeleteGroup_OnClicked();

	UE_API FText GetTextBoxValue() const;
	UE_API void OnTextBoxValueCommitted(const FText& InNewText, ETextCommit::Type InTextCommit);

	UE_API FText GetTextBoxTooltipText() const;
	UE_API FText GetTextBoxHintText() const;
	UE_API bool TextBox_OnVerifyTextChanged(const FText& InText, FText& OutErrorMessage);

	UE_API void SuggestionTextBox_GetSuggestions(const FString& Text, TArray<FString>& Suggestions);
	UE_API void SuggestionTextBox_GetHistory(TArray<FString>& Suggestions);

	UE_API void SuggestionTextBox_OnValueChanged(const FText& InNewText);
	UE_API FText SuggestionTextBox_GetValue() const;
	UE_API void SuggestionTextBox_OnValueCommitted(const FText& InNewText, ETextCommit::Type InTextCommit);

private:
	FFilterConfiguratorNodePtr FilterConfiguratorNodePtr;

	TSharedPtr<SComboBox<TSharedPtr<FFilter>>> FilterTypeComboBox;

	TSharedPtr<SComboBox<TSharedPtr<IFilterOperator>>> FilterOperatorComboBox;

	TSharedPtr<SComboBox<TSharedPtr<FFilterGroupOperator>>> FilterGroupOperatorComboBox;

	FString SuggestionTextBoxValue;
};

} // namespace UE::Insights

#undef UE_API
