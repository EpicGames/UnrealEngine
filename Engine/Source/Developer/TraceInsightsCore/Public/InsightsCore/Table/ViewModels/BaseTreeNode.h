// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#include "InsightsCore/Common/SimpleRtti.h"

#define UE_API TRACEINSIGHTSCORE_API

struct FSlateBrush;

namespace UE::Insights
{

class ITableCellValueSorter;
enum class ESortMode;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FBaseTreeNode;

/** Type definition for shared pointers to instances of FBaseTreeNode. */
typedef TSharedPtr<class FBaseTreeNode> FBaseTreeNodePtr;

/** Type definition for shared references to instances of FBaseTreeNode. */
typedef TSharedRef<class FBaseTreeNode> FBaseTreeNodeRef;

/** Type definition for shared references to const instances of FBaseTreeNode. */
typedef TSharedRef<const class FBaseTreeNode> FBaseTreeNodeRefConst;

/** Type definition for weak references to instances of FBaseTreeNode. */
typedef TWeakPtr<class FBaseTreeNode> FBaseTreeNodeWeak;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FBaseTreeNode : public TSharedFromThis<FBaseTreeNode>
{
	INSIGHTS_DECLARE_RTTI_BASE(FBaseTreeNode, UE_API)

protected:
	enum class EGroupNodeFlags : uint32
	{
		None = 0,
		IsExpanded = 1 << 0,
	};

	struct FGroupNodeData
	{
		/** Children of the group node. */
		TArray<FBaseTreeNodePtr> Children;

		/**
		 * A pointer to the filtered children of the group node.
		 * If filter is empty (no filtering), this points directly to Children array.
		 */
		TArray<FBaseTreeNodePtr>* FilteredChildrenPtr = &Children;

		/** Various flags specific to a group node. */
		EGroupNodeFlags Flags = EGroupNodeFlags::None;
	};

public:
	UE_NONCOPYABLE(FBaseTreeNode);

	/** Initialization constructor for the node. */
	explicit FBaseTreeNode(bool bInIsGroup)
		: GroupData(bInIsGroup ? new FGroupNodeData() : &DefaultGroupData)
	{
	}

	/** Initialization constructor for the node. */
	//UE_DEPRECATED(5.8, "Use the constructor without FName")
	explicit FBaseTreeNode(const FName InName, bool bInIsGroup)
		: Name(InName)
		, GroupData(bInIsGroup ? new FGroupNodeData() : &DefaultGroupData)
	{
	}

	virtual ~FBaseTreeNode()
	{
		RemoveGroupData();
	}

	/**
	 * @returns a name of this node.
	 */
	//UE_DEPRECATED(5.8, "Override the GetDisplayName() instead")
	const FName& GetName() const
	{
		return Name;
	}

	/**
	 * @returns a name of this node to display in tree view.
	 */
	UE_API virtual const FText GetDisplayName() const;

	/**
	 * @returns a name suffix for this node to display in tree view. Ex.: group nodes may include additional info, like the number of visible / total children.
	 */
	UE_API virtual const FText GetExtraDisplayName() const;

	/**
	 * @returns true if the node has an extra display name (a name suffix to display in tree view).
	 */
	UE_API virtual bool HasExtraDisplayName() const;

	/**
	 * @returns a descriptive text for this node to display in a tooltip.
	 */
	virtual const FText GetTooltipText() const
	{
		return FText::GetEmpty();
	}

	/**
	 * @returns the default icon for a group/leaf node.
	 */
	static UE_API const FSlateBrush* GetDefaultIcon(bool bIsGroupNode);

	/**
	 * @returns the brush icon for this node.
	 */
	virtual const FSlateBrush* GetIcon() const
	{
		return GetDefaultIcon(IsGroup());
	}

	/**
	 * @returns the default color tint for a group or leaf node.
	 */
	static UE_API FLinearColor GetDefaultIconColor(bool bIsGroupNode);

	/**
	 * @returns the color tint to be used for the icon of this node.
	 */
	virtual FLinearColor GetIconColor() const
	{
		return GetDefaultIconColor(IsGroup());
	}

	/**
	 * @returns the default color tint for a group or leaf node.
	 */
	static UE_API FLinearColor GetDefaultColor(bool bIsGroupNode);

	/**
	 * @returns the color tint to be used for the name text of this node.
	 */
	virtual FLinearColor GetColor() const
	{
		return GetDefaultColor(IsGroup());
	}

	//////////////////////////////////////////////////
	// Parent Node

	/**
	 * @returns a weak reference to the parent group of this node; may be invalid.
	 */
	FBaseTreeNodeWeak GetParentWeak() const
	{
		return Parent;
	}

	/**
	 * @returns a reference to the parent group of this node; may be invalid.
	 */
	FBaseTreeNodePtr GetParent() const
	{
		return Parent.Pin();
	}

	//////////////////////////////////////////////////
	// Group Node

	/**
	 * @returns true if this node is a group node.
	 */
	bool IsGroup() const
	{
		return GroupData != &DefaultGroupData;
	}

	/**
	 * Initializes the group data, allowing this node to accept children.
	 */
	void InitGroupData()
	{
		if (GroupData == &DefaultGroupData)
		{
			GroupData = new FGroupNodeData();
		}
	}

	void RemoveGroupData()
	{
		if (GroupData != &DefaultGroupData)
		{
			if (GroupData->FilteredChildrenPtr != &GroupData->Children)
			{
				delete GroupData->FilteredChildrenPtr;
			}

			delete GroupData;
			GroupData = &DefaultGroupData;
		}
	}

	//////////////////////////////////////////////////
	// Children Nodes

	/**
	 * @returns the number of children nodes.
	 */
	int32 GetChildrenCount() const
	{
		return GroupData->Children.Num();
	}

	/**
	 * Enumerates the children nodes.
	 */
	void EnumerateChildren(TFunction<bool(const FBaseTreeNodePtr&)> Callback) const
	{
		for (const FBaseTreeNodePtr& ChildNode : GroupData->Children)
		{
			if (!Callback(ChildNode))
			{
				break;
			}
		}
	}

	/**
	 * @returns a const reference to the children nodes of this group.
	 */
	const TArray<FBaseTreeNodePtr>& GetChildren() const
	{
		return GroupData->Children;
	}

	UE_API void SortChildren(const ITableCellValueSorter& Sorter, ESortMode SortMode);

	template <typename PredicateType>
	void SortChildren(PredicateType Predicate)
	{
		GroupData->Children.Sort(Predicate);
	}

	/** Adds specified node to the children nodes. */
	void AddChild(const FBaseTreeNodePtr& ChildPtr)
	{
		GroupData->Children.Add(ChildPtr);
	}

	/** Adds specified node to the children nodes. Also sets the current node as the parent group of the specified node. */
	void AddChildAndSetParent(const FBaseTreeNodePtr& ChildPtr)
	{
		ChildPtr->Parent = AsWeak();
		GroupData->Children.Add(ChildPtr);
	}

	/** Removes this node from the parent's children nodes (preserving the order). Also resets the parent group. */
	void RemoveFromParent()
	{
		FBaseTreeNodePtr ParentNodePtr = Parent.Pin();
		if (ParentNodePtr.IsValid())
		{
			ParentNodePtr->GroupData->Children.RemoveSingle(AsShared());
		}
		Parent.Reset();
	}

	/** Removes this node from the parent's children nodes (not preserving the order). Also resets the parent group. */
	void RemoveFromParentSwap()
	{
		FBaseTreeNodePtr ParentNodePtr = Parent.Pin();
		if (ParentNodePtr.IsValid())
		{
			ParentNodePtr->GroupData->Children.RemoveSwap(AsShared());
		}
		Parent.Reset();
	}

	/** Removes the specified node from the children nodes (preserving the order). If successful, resets the parent group of the specified node. */
	int32 RemoveChild(const FBaseTreeNodePtr& ChildPtr)
	{
		if (GroupData->Children.RemoveSingle(ChildPtr))
		{
			ChildPtr->Parent.Reset();
			return 1;
		}
		return 0;
	}

	/** Removes the specified node from the children nodes (not preserving the order). If successful, resets the parent group of the specified node. */
	int32 RemoveChildSwap(const FBaseTreeNodePtr& ChildPtr)
	{
		if (GroupData->Children.RemoveSwap(ChildPtr))
		{
			ChildPtr->Parent.Reset();
			return 1;
		}
		return 0;
	}

	void SetParent(const FBaseTreeNodeWeak& InParentWeak)
	{
		Parent = InParentWeak;
	}

	void SetParentForChildren()
	{
		FBaseTreeNodeWeak ThisNode = AsWeak();
		for (FBaseTreeNodePtr& NodePtr : GroupData->Children)
		{
			NodePtr->Parent = ThisNode;
		}
	}

	void SetParentForChildrenRec()
	{
		FBaseTreeNodeWeak ThisNode = AsWeak();
		for (FBaseTreeNodePtr& NodePtr : GroupData->Children)
		{
			NodePtr->Parent = ThisNode;
			NodePtr->SetParentForChildrenRec();
		}
	}

	void ResetParentForChildren()
	{
		for (FBaseTreeNodePtr& NodePtr : GroupData->Children)
		{
			NodePtr->Parent.Reset();
		}
	}

	void ResetParentForChildrenRec()
	{
		for (FBaseTreeNodePtr& NodePtr : GroupData->Children)
		{
			NodePtr->ResetParentForChildrenRec();
			NodePtr->Parent.Reset();
		}
	}

	/** Clears children. */
	void ClearChildren(int32 NewSize = 0)
	{
		ResetParentForChildren();
		GroupData->Children.Reset(NewSize);
	}

	void SwapChildren(TArray<FBaseTreeNodePtr>& NewChildren)
	{
		ResetParentForChildren();
		Swap(NewChildren, GroupData->Children);
		SetParentForChildren();
	}

	void SwapChildrenFast(TArray<FBaseTreeNodePtr>& NewChildren)
	{
		Swap(NewChildren, GroupData->Children);
	}

	//////////////////////////////////////////////////
	// Filtered Children Nodes

	virtual bool IsFiltered() const
	{
		return false;
	}

	/**
	 * @returns the number of filtered children nodes.
	 */
	int32 GetFilteredChildrenCount() const
	{
		return GroupData->FilteredChildrenPtr->Num();
	}

	/**
	 * @returns a const reference to the filtered child node at specified index.
	 */
	const FBaseTreeNodePtr& GetFilteredChildNode(int32 Index) const
	{
		return (*GroupData->FilteredChildrenPtr)[Index];
	}

	/**
	 * @returns a reference to the filtered child node at specified index.
	 */
	FBaseTreeNodePtr& GetFilteredChildNode(int32 Index)
	{
		return (*GroupData->FilteredChildrenPtr)[Index];
	}

	/**
	 * Enumerates the filtered children nodes.
	 */
	void EnumerateFilteredChildren(TFunction<bool(const FBaseTreeNodePtr&)> Callback) const
	{
		for (const FBaseTreeNodePtr& ChildNode : *GroupData->FilteredChildrenPtr)
		{
			if (!Callback(ChildNode))
			{
				break;
			}
		}
	}

	/**
	 * @returns a const reference to the children nodes that should be visible to the UI based on filtering.
	 */
	const TArray<FBaseTreeNodePtr>& GetFilteredChildren() const
	{
		return *GroupData->FilteredChildrenPtr;
	}

	UE_API void SortFilteredChildren(const ITableCellValueSorter& Sorter, ESortMode SortMode);

	template <typename PredicateType>
	void SortFilteredChildren(PredicateType Predicate)
	{
		GroupData->FilteredChildrenPtr->Sort(Predicate);
	}

	/** Clears (removes all) the filtered children nodes. */
	void ClearFilteredChildren(int32 NewSize = 0)
	{
		if (GroupData->FilteredChildrenPtr == &GroupData->Children)
		{
			GroupData->FilteredChildrenPtr = new TArray<FBaseTreeNodePtr>();
		}
		GroupData->FilteredChildrenPtr->Reset(NewSize);
	}

	/** Adds specified child to the filtered children nodes. */
	void AddFilteredChild(const FBaseTreeNodePtr& ChildPtr)
	{
		if (GroupData->FilteredChildrenPtr == &GroupData->Children)
		{
			GroupData->FilteredChildrenPtr = new TArray<FBaseTreeNodePtr>();
		}
		GroupData->FilteredChildrenPtr->Add(ChildPtr);
	}

	/**
	 * Resets the filtered children nodes.
	 * The filtered array of children nodes will be initialized with the unfiltered array of children.
	 */
	void ResetFilteredChildren()
	{
		if (GroupData->FilteredChildrenPtr != &GroupData->Children)
		{
			delete GroupData->FilteredChildrenPtr;
			GroupData->FilteredChildrenPtr = &GroupData->Children;
		}
	}

	/**
	 * Resets the filtered children for this node and also recursively for all children nodes.
	 * The filtered array of children nodes will be initialized with the unfiltered array of children.
	 */
	void ResetFilteredChildrenRec()
	{
		this->ResetFilteredChildren();
		for (FBaseTreeNodePtr Node : GroupData->Children)
		{
			Node->ResetFilteredChildrenRec();
		}
	}

	//////////////////////////////////////////////////

	bool IsExpanded() const
	{
		return ((uint32)GroupData->Flags & (uint32)EGroupNodeFlags::IsExpanded) != 0;
	}

	void SetExpansion(bool bOnOff)
	{
		if (IsGroup())
		{
			if (bOnOff)
			{
				GroupData->Flags = (EGroupNodeFlags)((uint32)GroupData->Flags | (uint32)EGroupNodeFlags::IsExpanded);
			}
			else
			{
				GroupData->Flags = (EGroupNodeFlags)((uint32)GroupData->Flags & ~(uint32)EGroupNodeFlags::IsExpanded);
			}
		}
	}

	//////////////////////////////////////////////////

	virtual uint32 GetDefaultSortOrder() const
	{
		return DefaultSortOrder;
	}

	//UE_DEPRECATED(5.8, "Override GetDefaultSortOrder() in a custom node instead")
	void SetDefaultSortOrder(uint32 Order)
	{
		DefaultSortOrder = Order;
	}

protected:
	/**
	 * @returns a reference to the children nodes of this group.
	 */
	TArray<FBaseTreeNodePtr>& GetChildrenMutable()
	{
		return GroupData->Children;
	}

private:
	/** The default sort order. Index used to optimize sorting. */
	uint32 DefaultSortOrder = 0; // deprecated in UE 5.8

	/** The name of this node. */
	const FName Name; // deprecated in UE 5.8

	/** A weak pointer to the parent group of this node. */
	FBaseTreeNodeWeak Parent;

	/** The struct containing properties of a group node. It is allocated only for group nodes. */
	FGroupNodeData* GroupData;

	/** The only group data for "not a group" nodes. */
	static UE_API FGroupNodeData DefaultGroupData;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef UE_API
