// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#define UE_API TRACEINSIGHTSCORE_API

namespace UE::Insights
{

class FBaseTreeNode;
class FTableColumn;

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Table View Model.
 * View model class for the STableListView and STableTreeView widgets.
 */
class FTable : public TSharedFromThis<FTable>
{
public:
	UE_API FTable();
	UE_API virtual ~FTable();

	const FText& GetDisplayName() const { return DisplayName; }
	void SetDisplayName(const FText& InDisplayName) { DisplayName = InDisplayName; }

	const FText& GetDescription() const { return Description; }
	void SetDescription(const FText& InDescription) { Description = InDescription; }

	UE_API virtual void Reset();

	int32 GetColumnCount() const { return Columns.Num(); }
	bool IsValid() const { return Columns.Num() > 0; }

	const TArray<TSharedRef<FTableColumn>>& GetColumns() const { return Columns; }
	UE_API void SetColumns(const TArray<TSharedRef<FTableColumn>>& InColumns);

	UE_API void GetVisibleColumns(TArray<TSharedRef<FTableColumn>>& InArray) const;

	TSharedRef<FTableColumn> FindColumnChecked(const FName& ColumnId) const
	{
		return ColumnIdToPtrMapping.FindChecked(ColumnId);
	}

	TSharedPtr<FTableColumn> FindColumn(const FName& ColumnId) const
	{
		const TSharedRef<FTableColumn>* const ColumnRefPtr = ColumnIdToPtrMapping.Find(ColumnId);
		if (ColumnRefPtr != nullptr)
		{
			return *ColumnRefPtr;
		}
		return nullptr;
	}

	UE_API int32 GetColumnPositionIndex(const FName& ColumnId) const;

	UE_API void GetVisibleColumnsData(const TArray<TSharedPtr<FBaseTreeNode>>& InNodes, const FName& LogListingName, TCHAR Separator, bool bIncludeHeaders, FString& OutData) const;

	static UE_API const FName GetHierarchyColumnId();

protected:
	void ResetColumns() { Columns.Reset(); }
	UE_API void AddColumn(TSharedRef<FTableColumn> Column);
	UE_API void AddHierarchyColumn(int32 ColumnIndex, const TCHAR* ColumnName);

private:
	FText DisplayName;
	FText Description;

	/** All available columns. */
	TArray<TSharedRef<FTableColumn>> Columns;

	/** Mapping between column Ids and FTableColumn shared refs. */
	TMap<FName, TSharedRef<FTableColumn>> ColumnIdToPtrMapping;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef UE_API
