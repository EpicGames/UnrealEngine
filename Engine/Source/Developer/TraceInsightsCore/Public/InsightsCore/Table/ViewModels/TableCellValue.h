// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/CString.h"
#include "Templates/SharedPointer.h"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class TRACEINSIGHTSCORE_API ETableCellDataType : uint32
{
	Unknown,

	// Basic types.
	Bool,
	//Int32,
	Int64,
	Float,
	Double,
	CString,

	// Custom types.
	Custom,
	Text, // FTextCustomTableCellValue

	/** Invalid enum type, may be used as a number of enumerations. */
	InvalidOrMax,
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class ICustomTableCellValue
{
public:
	virtual bool AsBool() const = 0;
	virtual int64 AsInt64() const = 0;
	virtual double AsDouble() const = 0;
	virtual FText AsText() const = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTextCustomTableCellValue : public ICustomTableCellValue
{
public:
	FTextCustomTableCellValue(const FText& InText) : Text(InText) {}
	virtual ~FTextCustomTableCellValue() = default;

	virtual bool AsBool() const override { return false; }
	virtual int64 AsInt64() const override { return 0; }
	virtual double AsDouble() const override { return 0.0; }
	virtual FText AsText() const override { return Text; }

	const FText& GetText() const { return Text; }

private:
	const FText Text;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FTableCellValue
{
public:
	FTableCellValue() : DataType(ETableCellDataType::Unknown) {}

	explicit FTableCellValue(bool Value) : DataType(ETableCellDataType::Bool), Bool(Value) {}
	explicit FTableCellValue(int64 Value) : DataType(ETableCellDataType::Int64), Int64(Value) {}
	explicit FTableCellValue(float Value) : DataType(ETableCellDataType::Float), Float(Value) {}
	explicit FTableCellValue(double Value) : DataType(ETableCellDataType::Double), Double(Value) {}
	explicit FTableCellValue(const TCHAR* Value) : DataType(ETableCellDataType::CString), CString(Value) {}
	explicit FTableCellValue(TSharedPtr<ICustomTableCellValue> Value, uint64 Id = 0) : DataType(ETableCellDataType::Custom), ValueId(Id), Custom(Value) {}
	explicit FTableCellValue(const FText& Value, uint64 Id = 0) : DataType(ETableCellDataType::Text), ValueId(Id), Custom(MakeShared<FTextCustomTableCellValue>(Value)) {}

	bool AsBool() const
	{
		switch (DataType)
		{
			case ETableCellDataType::Unknown:	return false;
			case ETableCellDataType::Bool:		return Bool;
			case ETableCellDataType::Int64:		return Int64 != 0;
			case ETableCellDataType::Float:		return Float != 0.0f;
			case ETableCellDataType::Double:	return Double != 0.0;
			case ETableCellDataType::CString:	return CString != nullptr;
			default:							return Custom.IsValid() ? Custom->AsBool() : false;
		}
	}

	int64 AsInt64() const
	{
		switch (DataType)
		{
			case ETableCellDataType::Unknown:	return 0;
			case ETableCellDataType::Bool:		return Bool ? 1 : 0;
			case ETableCellDataType::Int64:		return Int64;
			case ETableCellDataType::Float:		return static_cast<int64>(Float);
			case ETableCellDataType::Double:	return static_cast<int64>(Double);
			case ETableCellDataType::CString:	return FCString::Atoi64(CString);
			default:							return Custom.IsValid() ? Custom->AsInt64() : 0;
		}
	}

	float AsFloat() const
	{
		switch (DataType)
		{
			case ETableCellDataType::Unknown:	return 0.0f;
			case ETableCellDataType::Bool:		return Bool ? 1.0f : 0.0f;
			case ETableCellDataType::Int64:		return static_cast<float>(Int64);
			case ETableCellDataType::Float:		return Float;
			case ETableCellDataType::Double:	return static_cast<float>(Double);
			case ETableCellDataType::CString:	return FCString::Atof(CString);
			default:							return Custom.IsValid() ? static_cast<float>(Custom->AsDouble()) : 0.0f;
		}
	}

	double AsDouble() const
	{
		switch (DataType)
		{
			case ETableCellDataType::Unknown:	return 0.0;
			case ETableCellDataType::Bool:		return Bool ? 1.0 : 0.0;
			case ETableCellDataType::Int64:		return static_cast<double>(Int64);
			case ETableCellDataType::Float:		return static_cast<double>(Float);
			case ETableCellDataType::Double:	return Double;
			case ETableCellDataType::CString:	return FCString::Atod(CString);
			default:							return Custom.IsValid() ? Custom->AsDouble() : 0.0;
		}
	}

	FString AsString() const
	{
		switch (DataType)
		{
			case ETableCellDataType::Unknown:	return FString();
			case ETableCellDataType::Bool:		return Bool ? FString(TEXT("True")) : FString(TEXT("False"));
			case ETableCellDataType::Int64:		return FString::Printf(TEXT("%lld"), Int64);
			case ETableCellDataType::Float:		return FString::Printf(TEXT("%f"), Float);
			case ETableCellDataType::Double:	return FString::Printf(TEXT("%f"), Double);
			case ETableCellDataType::CString:	return FString(CString);
			default:							return Custom.IsValid() ? Custom->AsText().ToString() : FString();
		}
	}

	FText AsText() const
	{
		switch (DataType)
		{
			case ETableCellDataType::Unknown:	return FText::GetEmpty();
			case ETableCellDataType::Bool:		return Bool ? NSLOCTEXT("Table", "Bool_True", "True") : NSLOCTEXT("Table", "Bool_False", "False");
			case ETableCellDataType::Int64:		return FText::AsNumber(Int64);
			case ETableCellDataType::Float:		return FText::AsNumber(Float);
			case ETableCellDataType::Double:	return FText::AsNumber(Double);
			case ETableCellDataType::CString:	return FText::FromString(FString(CString));
			default:							return Custom.IsValid() ? Custom->AsText() : FText::GetEmpty();
		}
	}

	const FText& GetText() const
	{
		if (DataType == ETableCellDataType::Text && Custom.IsValid())
		{
			return StaticCastSharedPtr<FTextCustomTableCellValue>(Custom)->GetText();
		}
		return FText::GetEmpty();
	}

	uint64 GetValueId() const
	{
		return ValueId;
	}

public:
	ETableCellDataType DataType;

	union
	{
		bool Bool;
		int64 Int64;
		float Float;
		double Double;
		const TCHAR* CString; // should be valid for the lifetime of the owner table
		uint64 ValueId; // only used by Custom types
	};

	TSharedPtr<ICustomTableCellValue> Custom;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights
