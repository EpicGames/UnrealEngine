// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Layout/Visibility.h"
#include "Misc/Optional.h"

#include "InsightsCore/Table/ViewModels/TableCellValue.h"

#define UE_API TRACEINSIGHTSCORE_API

class IToolTip;
class SWidget;

namespace UE::Insights
{

class FBaseTreeNode;
class FTableColumn;

////////////////////////////////////////////////////////////////////////////////////////////////////

class ITableCellValueFormatter
{
public:
	virtual FText FormatValue(const TOptional<FTableCellValue>& InValue) const = 0;
	virtual FText FormatValueForTooltip(const TOptional<FTableCellValue>& InValue) const = 0;

	virtual FText FormatValue(const FTableColumn& Column, const FBaseTreeNode& Node) const = 0;
	virtual FText FormatValueForTooltip(const FTableColumn& Column, const FBaseTreeNode& Node) const = 0;
	virtual FText FormatValueForGrouping(const FTableColumn& Column, const FBaseTreeNode& Node) const = 0;

	virtual FText CopyValue(const FTableColumn& Column, const FBaseTreeNode& Node) const = 0;
	virtual FText CopyTooltip(const FTableColumn& Column, const FBaseTreeNode& Node) const = 0;

	virtual TSharedPtr<SWidget> GenerateCustomWidget(const FTableColumn& Column, const FBaseTreeNode& Node) const = 0;
	virtual TSharedPtr<IToolTip> GetCustomTooltip(const FTableColumn& Column, const FBaseTreeNode& Node) const = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTableCellValueFormatter : public ITableCellValueFormatter
{
public:
	FTableCellValueFormatter() {}
	virtual ~FTableCellValueFormatter() = default;

	virtual FText FormatValue(const TOptional<FTableCellValue>& InValue) const override { return FText::GetEmpty(); }
	virtual FText FormatValueForTooltip(const TOptional<FTableCellValue>& InValue) const override { return FormatValue(InValue); }

	UE_API virtual FText FormatValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override; // { return FormatValue(Column.GetValue(Node)); }
	UE_API virtual FText FormatValueForTooltip(const FTableColumn& Column, const FBaseTreeNode& Node) const override; // { return FormatValueForTooltip(Column.GetValue(Node)); }
	UE_API virtual FText FormatValueForGrouping(const FTableColumn& Column, const FBaseTreeNode& Node) const override; // { return FormatValueForTooltip(Column.GetValue(Node)); }

	UE_API virtual FText CopyValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override; // { return FormatValue(Column, Node); }
	UE_API virtual FText CopyTooltip(const FTableColumn& Column, const FBaseTreeNode& Node) const override; // { return FormatValueForTooltip(Column, Node); }

	virtual TSharedPtr<SWidget> GenerateCustomWidget(const FTableColumn& Column, const FBaseTreeNode& Node) const override { return nullptr; }
	UE_API virtual TSharedPtr<IToolTip> GetCustomTooltip(const FTableColumn& Column, const FBaseTreeNode& Node) const override;

	static UE_API EVisibility GetTooltipVisibility();
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTextValueFormatter : public FTableCellValueFormatter
{
public:
	virtual FText FormatValue(const TOptional<FTableCellValue>& InValue) const override
	{
		if (InValue.IsSet())
		{
			return InValue.GetValue().GetText();
		}
		return FText::GetEmpty();
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FAsTextValueFormatter : public FTableCellValueFormatter
{
public:
	virtual FText FormatValue(const TOptional<FTableCellValue>& InValue) const override
	{
		if (InValue.IsSet())
		{
			return InValue.GetValue().AsText();
		}
		return FText::GetEmpty();
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FBoolValueFormatterAsTrueFalse : public FTableCellValueFormatter
{
public:
	UE_API virtual FText FormatValue(const TOptional<FTableCellValue>& InValue) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FBoolValueFormatterAsOnOff : public FTableCellValueFormatter
{
public:
	UE_API virtual FText FormatValue(const TOptional<FTableCellValue>& InValue) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FInt64ValueFormatterAsNumber : public FTableCellValueFormatter
{
public:
	virtual FText FormatValue(const TOptional<FTableCellValue>& InValue) const override
	{
		return InValue.IsSet() ? FText::AsNumber(InValue.GetValue().Int64) : FText::GetEmpty();
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FInt64ValueFormatterAsUInt32InfiniteNumber : public FTableCellValueFormatter
{
public:
	UE_API virtual FText FormatValue(const TOptional<FTableCellValue>& InValue) const override;
	UE_API virtual FText FormatValueForTooltip(const TOptional<FTableCellValue>& InValue) const override;
};

UE_DEPRECATED(5.8, "Please fix the spelling for 'Infinte'")
typedef FInt64ValueFormatterAsUInt32InfiniteNumber FInt64ValueFormatterAsUInt32InfinteNumber;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FInt64ValueFormatterAsHex32 : public FTableCellValueFormatter
{
public:
	virtual FText FormatValue(const TOptional<FTableCellValue>& InValue) const override
	{
		return InValue.IsSet() ? FText::FromString(FString::Printf(TEXT("0x%08X"), static_cast<uint32>(InValue.GetValue().Int64))) : FText::GetEmpty();
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FInt64ValueFormatterAsHex64 : public FTableCellValueFormatter
{
public:
	virtual FText FormatValue(const TOptional<FTableCellValue>& InValue) const override
	{
		return InValue.IsSet() ? FText::FromString(FString::Printf(TEXT("0x%016llX"), static_cast<uint64>(InValue.GetValue().Int64))) : FText::GetEmpty();
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FInt64ValueFormatterAsMemory : public FTableCellValueFormatter
{
public:
	FInt64ValueFormatterAsMemory()
	{
		FormattingOptions.MaximumFractionalDigits = 1;
	}
	virtual ~FInt64ValueFormatterAsMemory() = default;

	const FNumberFormattingOptions& GetFormattingOptions() const { return FormattingOptions; }
	FNumberFormattingOptions& GetFormattingOptions() { return FormattingOptions; }

	UE_API virtual FText FormatValue(const TOptional<FTableCellValue>& InValue) const override;
	UE_API virtual FText FormatValueForTooltip(const TOptional<FTableCellValue>& InValue) const override;

	static UE_API FText FormatForTooltip(int64 InValue);

private:
	FNumberFormattingOptions FormattingOptions;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FFloatValueFormatterAsNumber : public FTableCellValueFormatter
{
public:
	UE_API virtual FText FormatValue(const TOptional<FTableCellValue>& InValue) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FFloatValueFormatterAsTimeAuto : public FTableCellValueFormatter
{
public:
	UE_API virtual FText FormatValue(const TOptional<FTableCellValue>& InValue) const override;
	UE_API virtual FText FormatValueForTooltip(const TOptional<FTableCellValue>& InValue) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FDoubleValueFormatterAsNumber : public FTableCellValueFormatter
{
public:
	UE_API virtual FText FormatValue(const TOptional<FTableCellValue>& InValue) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FDoubleValueFormatterAsCustomNumber : public FTableCellValueFormatter
{
public:
	FDoubleValueFormatterAsCustomNumber()
	{
		FormattingOptions.MaximumFractionalDigits = 2;
	}
	virtual ~FDoubleValueFormatterAsCustomNumber() = default;

	const FNumberFormattingOptions& GetFormattingOptions() const { return FormattingOptions; }
	FNumberFormattingOptions& GetFormattingOptions() { return FormattingOptions; }

	UE_API virtual FText FormatValue(const TOptional<FTableCellValue>& InValue) const override;

private:
	FNumberFormattingOptions FormattingOptions;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FDoubleValueFormatterAsPercent : public FTableCellValueFormatter
{
public:
	FDoubleValueFormatterAsPercent()
	{
		FormattingOptions.MinimumFractionalDigits = 2;
		FormattingOptions.MaximumFractionalDigits = 2;
	}
	virtual ~FDoubleValueFormatterAsPercent() = default;

	const FNumberFormattingOptions& GetFormattingOptions() const { return FormattingOptions; }
	FNumberFormattingOptions& GetFormattingOptions() { return FormattingOptions; }

	UE_API virtual FText FormatValue(const TOptional<FTableCellValue>& InValue) const override;

private:
	FNumberFormattingOptions FormattingOptions;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FDoubleValueFormatterAsTimeAuto : public FTableCellValueFormatter
{
public:
	UE_API virtual FText FormatValue(const TOptional<FTableCellValue>& InValue) const override;
	UE_API virtual FText FormatValueForTooltip(const TOptional<FTableCellValue>& InValue) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FDoubleValueFormatterAsTimeMs : public FTableCellValueFormatter
{
public:
	UE_API virtual FText FormatValue(const TOptional<FTableCellValue>& InValue) const override;
	UE_API virtual FText FormatValueForTooltip(const TOptional<FTableCellValue>& InValue) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FCStringValueFormatterAsText : public FTableCellValueFormatter
{
public:
	UE_API virtual FText FormatValue(const TOptional<FTableCellValue>& InValue) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef UE_API
