// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Misc/Optional.h"

#include "InsightsCore/Table/ViewModels/TableCellValue.h"

#define UE_API TRACEINSIGHTSCORE_API

namespace UE::Insights
{

class FBaseTreeNode;
class FTableColumn;

////////////////////////////////////////////////////////////////////////////////////////////////////

class ITableCellValueGetter
{
public:
	virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const = 0;
	virtual uint64 GetValueId(const FTableColumn& Column, const FBaseTreeNode& Node) const = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTableCellValueGetter : public ITableCellValueGetter
{
public:
	FTableCellValueGetter() {}
	virtual ~FTableCellValueGetter() = default;

	virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override { return TOptional<FTableCellValue>(); }
	virtual uint64 GetValueId(const FTableColumn& Column, const FBaseTreeNode& Node) const override { return 0; }
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FNameValueGetter : public FTableCellValueGetter
{
public:
	UE_API virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FDisplayNameValueGetter : public FTableCellValueGetter
{
public:
	UE_API virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef UE_API
