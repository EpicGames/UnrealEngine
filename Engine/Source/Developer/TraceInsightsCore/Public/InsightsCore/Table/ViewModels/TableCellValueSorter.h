// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "InsightsCore/Common/AsyncOperationProgress.h"
#include "InsightsCore/Table/ViewModels/TableCellValue.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"

#define UE_API TRACEINSIGHTSCORE_API

struct FSlateBrush;

namespace UE::Insights
{

class FBaseTreeNode;
typedef TSharedPtr<class FBaseTreeNode> FBaseTreeNodePtr;

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class TRACEINSIGHTSCORE_API ESortMode
{
	Ascending,
	Descending
};

////////////////////////////////////////////////////////////////////////////////////////////////////

typedef TFunction<bool(const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B)> FTreeNodeCompareFunc;

class ITableCellValueSorter
{
public:
	virtual FName GetName() const = 0;
	virtual FText GetShortName() const = 0;
	virtual FText GetTitleName() const = 0;
	virtual FText GetDescription() const = 0;
	virtual FName GetColumnId() const = 0;

	virtual FSlateBrush* GetIcon(ESortMode SortMode) const = 0;

	virtual FTreeNodeCompareFunc GetTreeNodeCompareDelegate(ESortMode SortMode) const = 0;

	virtual void Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const = 0;

	virtual void SetAsyncOperationProgress(IAsyncOperationProgress* AsyncOperationProgress) = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTableCellValueSorter : public ITableCellValueSorter
{
public:
	UE_API FTableCellValueSorter(const FName InName, const FText& InShortName, const FText& InTitleName, const FText& InDescription, TSharedRef<FTableColumn> InColumnRef);
	virtual ~FTableCellValueSorter() = default;

	virtual FName GetName() const override { return Name; }
	virtual FText GetShortName() const override { return ShortName; }
	virtual FText GetTitleName() const override { return TitleName; }
	virtual FText GetDescription() const override { return Description; }
	virtual FName GetColumnId() const override { return ColumnRef->GetId(); }

	virtual FSlateBrush* GetIcon(ESortMode SortMode) const override { return SortMode == ESortMode::Ascending ? AscendingIcon : DescendingIcon; }
	virtual FTreeNodeCompareFunc GetTreeNodeCompareDelegate(ESortMode SortMode) const override { return SortMode == ESortMode::Ascending ? AscendingCompareDelegate : DescendingCompareDelegate; }

	UE_API virtual void Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const override;

	virtual void SetAsyncOperationProgress(IAsyncOperationProgress* InAsyncOperationProgress) override { AsyncOperationProgress = InAsyncOperationProgress; }

	bool ShouldSortGroupsFirst() const { return bShouldSortGroupsFirst; }
	void SetShouldSortGroupsFirst(bool bOnOff) { bShouldSortGroupsFirst = bOnOff; }

protected:
	// Attempts to cancel the sort by throwing an exception. If exceptions are not available the return value is meant to be returned from sort predicates to speed up the sort.
	UE_API bool CancelSort() const;
	UE_API bool ShouldCancelSort() const;

protected:
	FName Name;
	FText ShortName;
	FText TitleName;
	FText Description;

	TSharedRef<FTableColumn> ColumnRef;

	FSlateBrush* AscendingIcon;
	FSlateBrush* DescendingIcon;

	FTreeNodeCompareFunc AscendingCompareDelegate;
	FTreeNodeCompareFunc DescendingCompareDelegate;

	IAsyncOperationProgress* AsyncOperationProgress = nullptr;

	bool bShouldSortGroupsFirst = true;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FBaseTableColumnSorter : public FTableCellValueSorter
{
public:
	UE_API FBaseTableColumnSorter(TSharedRef<FTableColumn> InColumnRef);
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FSorterByName : public FTableCellValueSorter
{
public:
	UE_API FSorterByName(TSharedRef<FTableColumn> InColumnRef);
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FSorterByTypeName : public FTableCellValueSorter
{
public:
	UE_API FSorterByTypeName(TSharedRef<FTableColumn> InColumnRef);
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FSorterByBoolValue : public FBaseTableColumnSorter
{
public:
	UE_API FSorterByBoolValue(TSharedRef<FTableColumn> InColumnRef);
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FSorterByInt64Value : public FBaseTableColumnSorter
{
public:
	UE_API FSorterByInt64Value(TSharedRef<FTableColumn> InColumnRef);

	UE_API virtual void Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FSorterByFloatValue : public FBaseTableColumnSorter
{
public:
	UE_API FSorterByFloatValue(TSharedRef<FTableColumn> InColumnRef);

	UE_API virtual void Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FSorterByDoubleValue : public FBaseTableColumnSorter
{
public:
	UE_API FSorterByDoubleValue(TSharedRef<FTableColumn> InColumnRef);

	UE_API virtual void Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FSorterByCStringValue : public FBaseTableColumnSorter
{
public:
	UE_API FSorterByCStringValue(TSharedRef<FTableColumn> InColumnRef);
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FSorterByTextValue : public FBaseTableColumnSorter
{
public:
	UE_API FSorterByTextValue(TSharedRef<FTableColumn> InColumnRef);

	UE_API virtual void Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FSorterByTextValueWithId : public FBaseTableColumnSorter
{
public:
	UE_API FSorterByTextValueWithId(TSharedRef<FTableColumn> InColumnRef);

	UE_API virtual void Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef UE_API
