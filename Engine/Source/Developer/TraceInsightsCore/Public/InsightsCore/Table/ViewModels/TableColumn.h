// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Types/SlateEnums.h"
#include "Misc/EnumClassFlags.h"
#include "Widgets/Views/SHeaderRow.h" // for EColumnSortMode

#include "InsightsCore/Table/ViewModels/TableCellValue.h"

#define UE_API TRACEINSIGHTSCORE_API

namespace UE::Insights
{

class FBaseTreeNode;
class IFilterValueConverter;
class FTable;
class ITableCellValueGetter;
class ITableCellValueFormatter;
class ITableCellValueSorter;

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class TRACEINSIGHTSCORE_API ETableColumnFlags : uint32
{
	None            = 0,
	ShouldBeVisible = (1 << 0), // Column should be initially visible.
	CanBeHidden     = (1 << 1), // Column can be hidden.
	CanBeFiltered   = (1 << 2), // Column can be used for filtering.
	IsHierarchy     = (1 << 3), // Column is the hierarchy (name) column, in a tree view.
	IsDynamic       = (1 << 4), // Column is dynamic. The column's value can not be cached.
};
ENUM_CLASS_FLAGS(ETableColumnFlags);

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class TRACEINSIGHTSCORE_API ETableColumnAggregation : uint32
{
	None = 0,
	Sum,
	Min,
	Max,
	SameValue
};

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Table Column View Model.
 * Holds information about a column in STableListView or STableTreeView widgets.
 */
class FTableColumn
{
public:
	/** No default constructor. */
	FTableColumn() = delete;

	/** Initialization constructor. */
	FTableColumn(const FName InId)
		: Id(InId)
		, Index(-1)
		, ShortName()
		, TitleName()
		, Description()
		, bIsVisible(false)
		, Flags(ETableColumnFlags::None)
		, HorizontalAlignment(HAlign_Left)
		, InitialWidth(60.0f)
		, MinWidth(0.0f)
		, MaxWidth(FLT_MAX)
		, DataType(ETableCellDataType::Unknown)
		, Aggregation(ETableColumnAggregation::None)
		, ValueGetter(GetDefaultValueGetter())
		, ValueFormatter(GetDefaultValueFormatter())
		, ValueSorter(nullptr)
		, InitialSortMode(EColumnSortMode::Ascending)
		, ParentTable(nullptr)
	{
	}

	//////////////////////////////////////////////////

	const FName& GetId() const { return Id; }

	int32 GetIndex() const { return Index; }
	void SetIndex(int32 InIndex) { Index = InIndex; }

	//////////////////////////////////////////////////
	// Name and Description

	const FText& GetShortName() const { return ShortName; }
	void SetShortName(const FText& InShortName) { ShortName = InShortName; }

	const FText& GetTitleName() const { return TitleName; }
	void SetTitleName(const FText& InTitleName) { TitleName = InTitleName; }

	const FText& GetDescription() const { return Description; }
	void SetDescription(const FText& InDescription) { Description = InDescription; }

	//////////////////////////////////////////////////
	// Visibility and other Flags

	bool IsVisible() const { return bIsVisible; }
	void Show() { bIsVisible = true; OnVisibilityChanged(); }
	void Hide() { bIsVisible = false; OnVisibilityChanged(); }
	void ToggleVisibility() { bIsVisible = !bIsVisible; OnVisibilityChanged(); }
	void SetVisibilityFlag(bool bOnOff) { bIsVisible = bOnOff; OnVisibilityChanged(); }
	/*virtual*/ void OnVisibilityChanged() {}

	/** Whether this column should be initially visible. */
	bool ShouldBeVisible() const { return EnumHasAnyFlags(Flags, ETableColumnFlags::ShouldBeVisible); }

	/** Whether this column can be hidden. */
	bool CanBeHidden() const { return EnumHasAnyFlags(Flags, ETableColumnFlags::CanBeHidden); }

	/** Whether this column can be used for filtering. */
	bool CanBeFiltered() const { return EnumHasAnyFlags(Flags, ETableColumnFlags::CanBeFiltered); }

	/** Whether this column is the hierarchy (name) column, in a tree view. */
	bool IsHierarchy() const { return EnumHasAnyFlags(Flags, ETableColumnFlags::IsHierarchy); }

	/** Whether this column is dynamic or not. If a column is dynamic, the column's value can not be cached. */
	bool IsDynamic() const { return EnumHasAnyFlags(Flags, ETableColumnFlags::IsDynamic); }

	void SetFlags(ETableColumnFlags InFlags) { Flags = InFlags; }

	//////////////////////////////////////////////////
	// Alignment and Width

	EHorizontalAlignment GetHorizontalAlignment() const { return HorizontalAlignment; }
	void SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment) { HorizontalAlignment = InHorizontalAlignment; }

	float GetInitialWidth() const { return InitialWidth; }
	void SetInitialWidth(float InWidth) { InitialWidth = InWidth; }

	float GetMinWidth() const { return MinWidth; }
	void SetMinWidth(float InWidth) { MinWidth = InWidth; }

	float GetMaxWidth() const { return MaxWidth; }
	void SetMaxWidth(float InWidth) { MaxWidth = InWidth; }

	//float GetWidth() const { return Width; }
	//void SetWidth(float InWidth) { Width = InWidth; }

	/** If MinWidth == MaxWidth, this column has fixed width and cannot be resized. */
	bool IsFixedWidth() const { return MinWidth == MaxWidth; }

	//////////////////////////////////////////////////
	// Data Type

	ETableCellDataType GetDataType() const { return DataType; }
	void SetDataType(ETableCellDataType InDataType) { DataType = InDataType; }

	//////////////////////////////////////////////////
	// Aggregation

	ETableColumnAggregation GetAggregation() const { return Aggregation; }
	void SetAggregation(ETableColumnAggregation InAggregation) { Aggregation = InAggregation; }

	//////////////////////////////////////////////////
	// Value Getter

	static UE_API TSharedRef<ITableCellValueGetter> GetDefaultValueGetter();

	TSharedRef<ITableCellValueGetter> GetValueGetter() const { return ValueGetter; }
	void SetValueGetter(TSharedRef<ITableCellValueGetter> InValueGetter) { ValueGetter = InValueGetter; }

	UE_API const TOptional<FTableCellValue> GetValue(const FBaseTreeNode& InNode) const;
	UE_API uint64 GetValueId(const FBaseTreeNode& InNode) const;

	//////////////////////////////////////////////////
	// Value Formatter

	static UE_API TSharedRef<ITableCellValueFormatter> GetDefaultValueFormatter();

	TSharedRef<ITableCellValueFormatter> GetValueFormatter() const { return ValueFormatter; }
	void SetValueFormatter(TSharedRef<ITableCellValueFormatter> InValueFormatter) { ValueFormatter = InValueFormatter; }

	UE_API FText GetValueAsText(const FBaseTreeNode& InNode) const;
	UE_API FText GetValueAsTooltipText(const FBaseTreeNode& InNode) const;
	UE_API FText GetValueAsGroupingText(const FBaseTreeNode& InNode) const;

	UE_API FText CopyValue(const FBaseTreeNode& InNode) const;
	UE_API FText CopyTooltip(const FBaseTreeNode& InNode) const;

	UE_API FString GetValueAsSerializableString(const FBaseTreeNode& InNode) const;

	//////////////////////////////////////////////////
	// Value Sorter (can be nullptr)

	TSharedPtr<ITableCellValueSorter> GetValueSorter() const { return ValueSorter; }
	void SetValueSorter(TSharedPtr<ITableCellValueSorter> InValueSorter) { ValueSorter = InValueSorter; }

	/** Whether this column can be used for sorting. */
	bool CanBeSorted() const { return ValueSorter.IsValid(); }

	/** Gets the initial sorting mode. */
	EColumnSortMode::Type GetInitialSortMode() const { return InitialSortMode; }

	/** Sets the initial sorting mode. */
	void SetInitialSortMode(EColumnSortMode::Type InMode) { InitialSortMode = InMode; }

	//////////////////////////////////////////////////
	// Value Converter (can be nullptr)

	TSharedPtr<IFilterValueConverter> GetValueConverter() const { return ValueConverter; }
	void SetValueConverter(TSharedPtr<IFilterValueConverter> InValueConverter) { ValueConverter = InValueConverter; }

	//////////////////////////////////////////////////

	TWeakPtr<FTable> GetParentTable() const { return ParentTable; }
	void SetParentTable(TWeakPtr<FTable> InParentTable) { ParentTable = InParentTable; }

	//////////////////////////////////////////////////

private:
	/** Id of the column. */
	FName Id;

	/** Column index in source table. */
	int32 Index;

	/** Short name of the column, displayed in the column header. */
	FText ShortName;

	/** Title name of the column, displayed as title in the column tooltip. */
	FText TitleName;

	/** Long name of the column, displayed in the column tooltip. */
	FText Description;

	/** Is this column visible? */
	bool bIsVisible;

	/** Other on/off switches. */
	ETableColumnFlags Flags;

	/** Horizontal alignment of the content in this column. */
	EHorizontalAlignment HorizontalAlignment;

	float InitialWidth; /**< Initial column width. */
	float MinWidth; /**< Minimum column width. */
	float MaxWidth; /**< Maximum column width. */
	//float Width; /**< Current column width. */

	/** Data type for this column. */
	ETableCellDataType DataType;

	/** Aggregation for values in this column, when grouped. */
	ETableColumnAggregation Aggregation; // TODO: make this an object (Aggregator)

	/** Custom getter for values identified by this column. */
	TSharedRef<ITableCellValueGetter> ValueGetter;

	/** Custom formatter for values displayed by this column. */
	TSharedRef<ITableCellValueFormatter> ValueFormatter;

	/** Custom sorter for values displayed by this column. */
	TSharedPtr<ITableCellValueSorter> ValueSorter;

	/** Initial sorting mode. */
	EColumnSortMode::Type InitialSortMode;

	/** Used in FilterConfigurator to convert from string to column's data type. */
	TSharedPtr<IFilterValueConverter> ValueConverter;

	/* Parent table. Only one table instance can own this column. */
	TWeakPtr<FTable> ParentTable;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef UE_API
