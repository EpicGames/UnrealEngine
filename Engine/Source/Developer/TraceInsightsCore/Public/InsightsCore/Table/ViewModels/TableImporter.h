// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceServices
#include "TraceServices/Model/TableImport.h"
#include "TraceServices/Containers/Tables.h"

#define UE_API TRACEINSIGHTSCORE_API

class SDockTab;
class FSpawnTabArgs;

namespace TraceServices
{
	struct FTableImportCallbackParams;
}

namespace UE::Insights
{

class SUntypedTableTreeView;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTableImporter : public TSharedFromThis<FTableImporter>
{
private:
	struct FOpenTableTabData
	{
		TSharedPtr<SDockTab> Tab;
		TSharedPtr<SUntypedTableTreeView> TableTreeView;

		bool operator ==(const FOpenTableTabData& Other) const
		{
			return Tab == Other.Tab && TableTreeView == Other.TableTreeView;
		}
	};

public:
	UE_API FTableImporter(FName InLogListingName);
	UE_API virtual ~FTableImporter();

	UE_API void StartImportProcess();
	UE_API void ImportFile(const FString& Filename);

	UE_API void StartDiffProcess();
	UE_API void DiffFiles(const FString& FilenameA, const FString& FilenameB);

	UE_API void CloseAllOpenTabs();

private:
	UE_API TSharedRef<SDockTab> SpawnTab_TableImportTreeView(const FSpawnTabArgs& Args, FName TableViewID, FText InDisplayName);
	UE_API TSharedRef<SDockTab> SpawnTab_TableDiffTreeView(const FSpawnTabArgs& Args, FName TableViewID, FText InDisplayName);
	UE_API void OnTableImportTreeViewTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	UE_API void DisplayImportTable(FName TableViewID);
	UE_API void DisplayDiffTable(FName TableViewID);
	UE_API void TableImportServiceCallback(TSharedPtr<TraceServices::FTableImportCallbackParams> Params);

	UE_API FName GetTableID(const FString& Path);

private:
	FName LogListingName;
	TMap<FName, FOpenTableTabData> OpenTablesMap;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef UE_API
