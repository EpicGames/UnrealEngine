// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "InsightsCore/Table/ViewModels/BaseTreeNode.h"
#include "InsightsCore/Table/ViewModels/TableCellValue.h"

#define UE_API TRACEINSIGHTSCORE_API

namespace UE::Insights
{

class FTable;
class FTreeNodeGrouping;
class STableTreeView;

struct FTableRowId
{
	static constexpr int32 InvalidRowIndex = -1;

	FTableRowId(int32 InRowIndex) : RowIndex(InRowIndex) {}

	bool HasValidIndex() const { return RowIndex >= 0; }

	int32 RowIndex;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTableTreeNode;

/** Type definition for shared pointers to instances of FTableTreeNode. */
typedef TSharedPtr<class FTableTreeNode> FTableTreeNodePtr;

/** Type definition for shared references to instances of FTableTreeNode. */
typedef TSharedRef<class FTableTreeNode> FTableTreeNodeRef;

/** Type definition for shared references to const instances of FTableTreeNode. */
typedef TSharedRef<const class FTableTreeNode> FTableTreeNodeRefConst;

/** Type definition for weak references to instances of FTableTreeNode. */
typedef TWeakPtr<class FTableTreeNode> FTableTreeNodeWeak;

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Table Tree Node View Model.
 * Class used to store information about a generic table tree node (used in STableTreeView).
 */
class FTableTreeNode : public FBaseTreeNode
{
	INSIGHTS_DECLARE_RTTI(FTableTreeNode, FBaseTreeNode, UE_API)

public:
	UE_NONCOPYABLE(FTableTreeNode);

	/** Initialization constructor for a table record node. */
	explicit FTableTreeNode(TWeakPtr<FTable> InParentTable, int32 InRowIndex, bool IsGroup = false)
		: FBaseTreeNode(IsGroup)
		, ParentTable(InParentTable)
		, RowId(InRowIndex)
	{
	}

	/** Initialization constructor for a group node. */
	explicit FTableTreeNode(TWeakPtr<FTable> InParentTable)
		: FBaseTreeNode(true)
		, ParentTable(InParentTable)
		, RowId(FTableRowId::InvalidRowIndex)
	{
	}

	//////////////////////////////////////////////////
	// Deprecated constructors

	/** Initialization constructor for a table record node. */
	//UE_DEPRECATED(5.8, "Use the constructor without FName.")
	UE_API explicit FTableTreeNode(const FName InName, TWeakPtr<FTable> InParentTable, int32 InRowIndex);

	/** Initialization constructor for a group node. */
	//UE_DEPRECATED(5.8, "Use the constructor without FName.")
	UE_API explicit FTableTreeNode(const FName InGroupName, TWeakPtr<FTable> InParentTable);

	/** Initialization constructor for a table record node. */
	//UE_DEPRECATED(5.8, "Use the constructor without FName.")
	UE_API explicit FTableTreeNode(const FName InName, TWeakPtr<FTable> InParentTable, int32 InRowIndex, bool IsGroup);

	//////////////////////////////////////////////////

	virtual ~FTableTreeNode()
	{
		CleanupAggregatedValues();
	}

	const TWeakPtr<FTable>& GetParentTable() const { return ParentTable; }
	FTableRowId GetRowId() const { return RowId; }
	int32 GetRowIndex() const { return RowId.RowIndex; }

	UE_API virtual const FText GetDisplayName() const override;

	//////////////////////////////////////////////////
	// Aggregation

	void InitAggregatedValues()
	{
		if (!AggregatedValues)
		{
			AggregatedValues = new TMap<FName, FTableCellValue>();
		}
	}

	void CleanupAggregatedValues()
	{
		if (AggregatedValues)
		{
			delete AggregatedValues;
			AggregatedValues = nullptr;
		}
	}

	void ResetAggregatedValues()
	{
		CleanupAggregatedValues();
	}

	void ResetAggregatedValue(const FName& ColumnId)
	{
		if (AggregatedValues)
		{
			AggregatedValues->Remove(ColumnId);
		}
	}

	bool HasAggregatedValue(const FName& ColumnId) const
	{
		return AggregatedValues && AggregatedValues->Contains(ColumnId);
	}

	const FTableCellValue* FindAggregatedValue(const FName& ColumnId) const
	{
		return AggregatedValues ? AggregatedValues->Find(ColumnId) : nullptr;
	}

	const FTableCellValue& GetAggregatedValue(const FName& ColumnId) const
	{
		checkSlow(AggregatedValues != nullptr);
		return AggregatedValues->FindChecked(ColumnId);
	}

	void SetAggregatedValue(const FName& ColumnId, const FTableCellValue& Value)
	{
		InitAggregatedValues();
		AggregatedValues->Add(ColumnId, Value);
	}

	//////////////////////////////////////////////////

	virtual uint32 GetDefaultSortOrder() const override
	{
		return uint32(RowId.RowIndex + 1);
	}

	virtual bool IsFiltered() const { return bIsFiltered; }
	virtual void SetIsFiltered(bool InValue) { bIsFiltered = InValue; }

	virtual bool OnLazyCreateChildren(TSharedPtr<STableTreeView> InTableTreeView) { return false; }

	/**
	 * The grouping that has generated this node.
	 * This is used to correctly apply further groupings for the lazy created children.
	 * If this returns nullptr, grouping is not applied for lazy created children.
	 */
	virtual const FTreeNodeGrouping* GetAuthorGrouping() const { return nullptr; }

protected:
	TWeakPtr<FTable> ParentTable;
	FTableRowId RowId = FTableRowId::InvalidRowIndex;
	bool bIsFiltered = false;
	TMap<FName, FTableCellValue>* AggregatedValues = nullptr;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FCustomTableTreeNode : public FTableTreeNode
{
	INSIGHTS_DECLARE_RTTI(FCustomTableTreeNode, FTableTreeNode, UE_API)

public:
	/** Initialization constructor for a table record node. */
	explicit FCustomTableTreeNode(TWeakPtr<FTable> InParentTable, int32 InRowIndex, const FText& InDisplayName, bool IsGroup)
		: FTableTreeNode(InParentTable, InRowIndex, IsGroup)
		, DisplayName(InDisplayName)
		, IconBrush(FBaseTreeNode::GetDefaultIcon(true))
		, IconColor(FBaseTreeNode::GetDefaultIconColor(true))
		, Color(FBaseTreeNode::GetDefaultColor(true))
	{
	}

	/** Initialization constructor for a table record node. */
	explicit FCustomTableTreeNode(TWeakPtr<FTable> InParentTable, int32 InRowIndex, const FText& InDisplayName, const FSlateBrush* InIconBrush, FLinearColor InIconColor, FLinearColor InColor, bool IsGroup)
		: FTableTreeNode(InParentTable, InRowIndex, IsGroup)
		, DisplayName(InDisplayName)
		, IconBrush(InIconBrush)
		, IconColor(InIconColor)
		, Color(InColor)
	{
	}

	/** Initialization constructor for a table record node. */
	explicit FCustomTableTreeNode(TWeakPtr<FTable> InParentTable, int32 InRowIndex, const FSlateBrush* InIconBrush, FLinearColor InIconColor, FLinearColor InColor, bool IsGroup)
		: FTableTreeNode(InParentTable, InRowIndex, IsGroup)
		, DisplayName()
		, IconBrush(InIconBrush)
		, IconColor(InIconColor)
		, Color(InColor)
	{
	}

	/** Initialization constructor for the group node. */
	explicit FCustomTableTreeNode(TWeakPtr<FTable> InParentTable, const FText& InDisplayName)
		: FTableTreeNode(InParentTable)
		, DisplayName(InDisplayName)
		, IconBrush(FBaseTreeNode::GetDefaultIcon(true))
		, IconColor(FBaseTreeNode::GetDefaultIconColor(true))
		, Color(FBaseTreeNode::GetDefaultColor(true))
	{
	}

	/** Initialization constructor for the group node. */
	explicit FCustomTableTreeNode(TWeakPtr<FTable> InParentTable, const FText& InDisplayName, const FSlateBrush* InIconBrush, FLinearColor InIconColor, FLinearColor InColor)
		: FTableTreeNode(InParentTable)
		, DisplayName(InDisplayName)
		, IconBrush(InIconBrush)
		, IconColor(InIconColor)
		, Color(InColor)
	{
	}

	/** Initialization constructor for the group node. */
	explicit FCustomTableTreeNode(TWeakPtr<FTable> InParentTable, const FSlateBrush* InIconBrush, FLinearColor InIconColor, FLinearColor InColor)
		: FTableTreeNode(InParentTable)
		, DisplayName()
		, IconBrush(InIconBrush)
		, IconColor(InIconColor)
		, Color(InColor)
	{
	}

	//////////////////////////////////////////////////
	// Deprecated constructors

	/** Initialization constructor for a table record node. */
	//UE_DEPRECATED(5.8, "Use the constructor without FName. Set name using SetDisplayName().")
	UE_API explicit FCustomTableTreeNode(const FName InName, TWeakPtr<FTable> InParentTable, int32 InRowIndex, const FSlateBrush* InIconBrush, FLinearColor InColor, bool IsGroup);

	/** Initialization constructor for a table record node. */
	//UE_DEPRECATED(5.8, "Use the constructor without FName. Set name using SetDisplayName().")
	UE_API explicit FCustomTableTreeNode(const FName InName, TWeakPtr<FTable> InParentTable, int32 InRowIndex, const FSlateBrush* InIconBrush, FLinearColor InIconColor, FLinearColor InColor, bool IsGroup);

	/** Initialization constructor for the group node. */
	//UE_DEPRECATED(5.8, "Use the constructor without FName. Set name using SetDisplayName().")
	UE_API explicit FCustomTableTreeNode(const FName InName, TWeakPtr<FTable> InParentTable, FLinearColor InColor);

	/** Initialization constructor for the group node. */
	//UE_DEPRECATED(5.8, "Use the constructor without FName. Set name using SetDisplayName().")
	UE_API explicit FCustomTableTreeNode(const FName InName, TWeakPtr<FTable> InParentTable, const FSlateBrush* InIconBrush, FLinearColor InColor);

	/** Initialization constructor for the group node. */
	//UE_DEPRECATED(5.8, "Use the constructor without FName. Set name using SetDisplayName().")
	UE_API explicit FCustomTableTreeNode(const FName InName, TWeakPtr<FTable> InParentTable, const FSlateBrush* InIconBrush, FLinearColor InIconColor, FLinearColor InColor);

	//////////////////////////////////////////////////

	virtual ~FCustomTableTreeNode() = default;

	//////////////////////////////////////////////////
	// Display Name

	virtual const FText GetDisplayName() const override
	{
		return DisplayName;
	}

	/**
	 * Sets the display name for this node.
	 */
	void SetDisplayName(const FText& InDisplayName)
	{
		DisplayName = InDisplayName;
	}

	/**
	 * Sets the display name for this node.
	 */
	void SetDisplayName(FText&& InDisplayName)
	{
		DisplayName = InDisplayName;
	}

	//////////////////////////////////////////////////
	// Icon

	virtual const FSlateBrush* GetIcon() const override
	{
		return IconBrush;
	}

	/**
	 * Sets an icon brush for this node.
	 */
	void SetIcon(const FSlateBrush* InIconBrush)
	{
		IconBrush = InIconBrush;
	}

	//////////////////////////////////////////////////
	// Icon Color

	virtual FLinearColor GetIconColor() const override
	{
		return IconColor;
	}

	/**
	 * Sets the color tint for the icon of this node.
	 */
	void SetIconColor(const FLinearColor& InIconColor)
	{
		IconColor = InIconColor;
	}

	//////////////////////////////////////////////////
	// Text Color

	virtual FLinearColor GetColor() const override
	{
		return Color;
	}

	/**
	 * Sets the color tint for the display name of this node.
	 */
	void SetColor(const FLinearColor& InColor)
	{
		Color = InColor;
	}

	//////////////////////////////////////////////////

private:
	/** The display name of this node. */
	FText DisplayName;

	/** The icon of this node. */
	const FSlateBrush* IconBrush;

	/** The color tint for the icon of this node. */
	FLinearColor IconColor;

	/** The color tint for the display name of this node. */
	FLinearColor Color;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef UE_API
