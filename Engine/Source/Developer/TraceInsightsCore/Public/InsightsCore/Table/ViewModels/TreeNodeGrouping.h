// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Delegates/DelegateCombinations.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#include "InsightsCore/Common/AsyncOperationProgress.h"
#include "InsightsCore/Common/SimpleRtti.h"
#include "InsightsCore/Table/ViewModels/BaseTreeNode.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"
#include "InsightsCore/Table/ViewModels/TableTreeNode.h"

#include <atomic>

#define UE_API TRACEINSIGHTSCORE_API

struct FSlateBrush;

namespace UE::Insights
{

class FTable;

////////////////////////////////////////////////////////////////////////////////////////////////////

class ITreeNodeGrouping
{
	INSIGHTS_DECLARE_RTTI_BASE(ITreeNodeGrouping, UE_API)

public:
	virtual FText GetShortName() const = 0;
	virtual FText GetTitleName() const = 0;
	virtual FText GetDescription() const = 0;

	virtual const FSlateBrush* GetIcon() const = 0;
	virtual const FLinearColor& GetColor() const = 0;

	virtual FName GetColumnId() const = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTreeNodeGroupInfo
{
public:
	FTreeNodeGroupInfo() = default;

	FTreeNodeGroupInfo(const FName& InName, bool bInIsExpanded = false)
		: Name(InName)
		, bIsExpanded(bInIsExpanded)
	{
	}

	FTreeNodeGroupInfo(const FName& InName, bool bInIsExpanded, const FLinearColor& InColor)
		: Name(InName)
		, bIsExpanded(bInIsExpanded)
		, bIsCustom(true)
		, Icon(nullptr)
		, IconColor(InColor)
		, Color(InColor)
	{
	}

	FTreeNodeGroupInfo(const FName& InName, bool bInIsExpanded, const FSlateBrush* InIcon, const FLinearColor& InIconColor, const FLinearColor& InColor)
		: Name(InName)
		, bIsExpanded(bInIsExpanded)
		, bIsCustom(true)
		, Icon(InIcon)
		, IconColor(InIconColor)
		, Color(InColor)
	{
	}

	const FName& GetName() const { return Name; }
	FText GetDisplayName() const { return FText::FromName(Name); }

	bool IsExpanded() const { return bIsExpanded; }
	bool IsCustom() const { return bIsCustom; }

	const FSlateBrush* GetIcon() const { return Icon; }
	const FLinearColor& GetIconColor() const { return IconColor; }
	const FLinearColor& GetColor() const { return Color; }

private:
	FName Name;
	bool bIsExpanded = false;
	bool bIsCustom = false;
	const FSlateBrush* Icon = nullptr;
	FLinearColor IconColor;
	FLinearColor Color;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTreeNodeGrouping : public ITreeNodeGrouping
{
	INSIGHTS_DECLARE_RTTI(FTreeNodeGrouping, ITreeNodeGrouping, UE_API)

public:
	UE_API FTreeNodeGrouping();
	UE_API FTreeNodeGrouping(const FText& InShortName, const FText& InTitleName, const FText& InDescription, const FSlateBrush* InIcon);

	FTreeNodeGrouping(const FTreeNodeGrouping&) = delete;
	FTreeNodeGrouping& operator=(FTreeNodeGrouping&) = delete;

	virtual ~FTreeNodeGrouping() = default;

	// ITreeNodeGrouping
	virtual FText GetShortName() const override { return ShortName; }
	virtual FText GetTitleName() const override { return TitleName; }
	virtual FText GetDescription() const override { return Description; }
	virtual const FSlateBrush* GetIcon() const override { return Icon; }
	virtual const FLinearColor& GetColor() const override { return Color; }
	virtual FName GetColumnId() const override { return NAME_None; }

	virtual void SetIcon(const FSlateBrush* InIcon) { Icon = InIcon; }
	virtual void SetColor(const FLinearColor& InColor) { Color = InColor; }

	virtual FTreeNodeGroupInfo GetGroupForNode(const FBaseTreeNodePtr InNode) const { return FTreeNodeGroupInfo(); }
	UE_API virtual void GroupNodes(const TArray<FTableTreeNodePtr>& Nodes, FTableTreeNode& ParentGroup, TWeakPtr<FTable> InParentTable, IAsyncOperationProgress& InAsyncOperationProgress) const;

	static UE_API FLinearColor MakeDistinctColor(const FName& InName);

protected:
	FText ShortName;
	FText TitleName;
	FText Description;
	const FSlateBrush* Icon = nullptr;
	FLinearColor Color;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Creates a single group for all nodes. */
class FTreeNodeGroupingFlat : public FTreeNodeGrouping
{
	INSIGHTS_DECLARE_RTTI(FTreeNodeGroupingFlat, FTreeNodeGrouping, UE_API)

public:
	UE_API FTreeNodeGroupingFlat();
	virtual ~FTreeNodeGroupingFlat() = default;

	UE_API virtual void GroupNodes(const TArray<FTableTreeNodePtr>& Nodes, FTableTreeNode& ParentGroup, TWeakPtr<FTable> InParentTable, IAsyncOperationProgress& InAsyncOperationProgress) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Creates a group for each unique value. */
class FTreeNodeGroupingByUniqueValue : public FTreeNodeGrouping
{
	INSIGHTS_DECLARE_RTTI(FTreeNodeGroupingByUniqueValue, FTreeNodeGrouping, UE_API);

public:
	UE_API FTreeNodeGroupingByUniqueValue(TSharedRef<FTableColumn> InColumnRef);
	virtual ~FTreeNodeGroupingByUniqueValue() = default;

	UE_API virtual FTreeNodeGroupInfo GetGroupForNode(const FBaseTreeNodePtr InNode) const override;

	virtual FName GetColumnId() const override { return ColumnRef->GetId(); }
	TSharedRef<FTableColumn> GetColumn() const { return ColumnRef; }

protected:
	TSharedRef<FCustomTableTreeNode> CreateGroupNode(TWeakPtr<FTable> InParentTable, const FText& InGroupName) const;
	TSharedRef<FCustomTableTreeNode> CreateUnsetGroupNode(TWeakPtr<FTable> InParentTable) const;

private:
	TSharedRef<FTableColumn> ColumnRef;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Creates a group for each unique value (assumes data type of cell values is a simple type). */
template<typename Type>
class TTreeNodeGroupingByUniqueValue : public FTreeNodeGroupingByUniqueValue
{
public:
	TTreeNodeGroupingByUniqueValue(TSharedRef<FTableColumn> InColumnRef) : FTreeNodeGroupingByUniqueValue(InColumnRef) {}
	virtual ~TTreeNodeGroupingByUniqueValue() = default;

	virtual void GroupNodes(const TArray<FTableTreeNodePtr>& Nodes, FTableTreeNode& ParentGroup, TWeakPtr<FTable> InParentTable, IAsyncOperationProgress& InAsyncOperationProgress) const override;

private:
	static Type GetValue(const FTableCellValue& CellValue);
};

template<typename Type>
void TTreeNodeGroupingByUniqueValue<Type>::GroupNodes(const TArray<FTableTreeNodePtr>& Nodes, FTableTreeNode& ParentGroup, TWeakPtr<FTable> InParentTable, IAsyncOperationProgress& InAsyncOperationProgress) const
{
	TMap<Type, FTableTreeNodePtr> GroupMap;
	FTableTreeNodePtr UnsetGroupPtr = nullptr;

	ParentGroup.ClearChildren();

	for (FTableTreeNodePtr NodePtr : Nodes)
	{
		if (InAsyncOperationProgress.ShouldCancelAsyncOp())
		{
			return;
		}

		if (NodePtr->IsGroup())
		{
			ParentGroup.AddChildAndSetParent(NodePtr);
			continue;
		}

		FTableTreeNodePtr GroupPtr = nullptr;

		const FTableColumn& Column = *GetColumn();
		const TOptional<FTableCellValue> CellValue = Column.GetValue(*NodePtr);
		if (CellValue.IsSet())
		{
			const Type Value = GetValue(CellValue.GetValue());

			FTableTreeNodePtr* GroupPtrPtr = GroupMap.Find(Value);
			if (!GroupPtrPtr)
			{
				const FText GroupName = Column.GetValueAsGroupingText(*NodePtr);
				GroupPtr = CreateGroupNode(InParentTable, GroupName);
				ParentGroup.AddChildAndSetParent(GroupPtr);
				GroupMap.Add(Value, GroupPtr);
			}
			else
			{
				GroupPtr = *GroupPtrPtr;
			}
		}
		else
		{
			if (!UnsetGroupPtr)
			{
				UnsetGroupPtr = CreateUnsetGroupNode(InParentTable);
				ParentGroup.AddChildAndSetParent(UnsetGroupPtr);
			}
			GroupPtr = UnsetGroupPtr;
		}

		GroupPtr->AddChildAndSetParent(NodePtr);
	}
}

typedef TTreeNodeGroupingByUniqueValue<bool> FTreeNodeGroupingByUniqueValueBool;
typedef TTreeNodeGroupingByUniqueValue<int64> FTreeNodeGroupingByUniqueValueInt64;
typedef TTreeNodeGroupingByUniqueValue<float> FTreeNodeGroupingByUniqueValueFloat;
typedef TTreeNodeGroupingByUniqueValue<double> FTreeNodeGroupingByUniqueValueDouble;

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Creates a group for each unique value (assumes data type of cell values is const TCHAR*). */
class FTreeNodeGroupingByUniqueValueCString : public FTreeNodeGroupingByUniqueValue
{
public:
	FTreeNodeGroupingByUniqueValueCString(TSharedRef<FTableColumn> InColumnRef) : FTreeNodeGroupingByUniqueValue(InColumnRef) {}
	virtual ~FTreeNodeGroupingByUniqueValueCString() = default;

	UE_API virtual void GroupNodes(const TArray<FTableTreeNodePtr>& Nodes, FTableTreeNode& ParentGroup, TWeakPtr<FTable> InParentTable, IAsyncOperationProgress& InAsyncOperationProgress) const override;

private:
	static UE_API FName GetGroupName(const TCHAR* Value);
};

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Creates a group for each first letter of node names. */
class FTreeNodeGroupingByNameFirstLetter : public FTreeNodeGrouping
{
	INSIGHTS_DECLARE_RTTI(FTreeNodeGroupingByNameFirstLetter, FTreeNodeGrouping, UE_API);

public:
	UE_API FTreeNodeGroupingByNameFirstLetter();
	virtual ~FTreeNodeGroupingByNameFirstLetter() = default;

	UE_API virtual FTreeNodeGroupInfo GetGroupForNode(const FBaseTreeNodePtr InNode) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Creates a group for each type. */
class FTreeNodeGroupingByType : public FTreeNodeGrouping
{
	INSIGHTS_DECLARE_RTTI(FTreeNodeGroupingByType, FTreeNodeGrouping, UE_API);

public:
	UE_API FTreeNodeGroupingByType();
	virtual ~FTreeNodeGroupingByType() = default;

	UE_API virtual FTreeNodeGroupInfo GetGroupForNode(const FBaseTreeNodePtr InNode) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Creates a tree hierarchy out of the path structure of string values. */
class FTreeNodeGroupingByPathBreakdown : public FTreeNodeGrouping
{
	INSIGHTS_DECLARE_RTTI(FTreeNodeGroupingByPathBreakdown, FTreeNodeGrouping, UE_API);

public:
	UE_API FTreeNodeGroupingByPathBreakdown(TSharedRef<FTableColumn> InColumnRef);
	virtual ~FTreeNodeGroupingByPathBreakdown() = default;

	UE_API virtual void GroupNodes(const TArray<FTableTreeNodePtr>& Nodes, FTableTreeNode& ParentGroup, TWeakPtr<FTable> InParentTable, IAsyncOperationProgress& InAsyncOperationProgress) const override;

	virtual FName GetColumnId() const override { return ColumnRef->GetId(); }
	TSharedRef<FTableColumn> GetColumn() const { return ColumnRef; }

private:
	TSharedRef<FTableColumn> ColumnRef;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef UE_API
