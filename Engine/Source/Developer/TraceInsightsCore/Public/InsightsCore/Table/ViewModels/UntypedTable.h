// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Templates/SharedPointer.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/Table.h"

#define UE_API TRACEINSIGHTSCORE_API

namespace TraceServices
{
	class ITableLayout;
	class IUntypedTable;
	class IUntypedTableReader;
}

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FUntypedTable : public FTable
{
public:
	UE_API FUntypedTable();
	UE_API virtual ~FUntypedTable();

	UE_API virtual void Reset();

	TSharedPtr<TraceServices::IUntypedTable> GetSourceTable() const { return SourceTable; }
	TSharedPtr<TraceServices::IUntypedTableReader> GetTableReader() const { return TableReader; }

	/* Update table content. Returns true if the table layout has changed. */
	UE_API bool UpdateSourceTable(TSharedPtr<TraceServices::IUntypedTable> InSourceTable);

private:
	UE_API void CreateColumns(const TraceServices::ITableLayout& TableLayout);

private:
	TSharedPtr<TraceServices::IUntypedTable> SourceTable;
	TSharedPtr<TraceServices::IUntypedTableReader> TableReader;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef UE_API
