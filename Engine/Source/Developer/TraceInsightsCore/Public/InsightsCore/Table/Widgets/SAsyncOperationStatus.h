// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API TRACEINSIGHTSCORE_API

namespace UE::Insights
{

class IAsyncOperationStatusProvider;

// A widget representing the status message used in table / tree views for background operations.
class SAsyncOperationStatus : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAsyncOperationStatus) {}

	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, TSharedRef<IAsyncOperationStatusProvider> InStatusProvider);

private:
	UE_API EVisibility GetContentVisibility() const;

	UE_API FSlateColor GetBackgroundColorAndOpacity() const;
	UE_API FSlateColor GetTextColorAndOpacity() const;
	UE_API float ComputeOpacity() const;

	UE_API FText GetText() const;
	UE_API FText GetAnimatedText() const;
	UE_API FText GetTooltipText() const;

private:
	TWeakPtr<IAsyncOperationStatusProvider> StatusProvider;
};

} // namespace UE::Insights

#undef UE_API
