// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/FilterCollection.h"
#include "Misc/TextFilter.h"
#include "SlateFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Navigation/SBreadcrumbTrail.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

#include "InsightsCore/Common/AsyncOperationProgress.h"
#include "InsightsCore/Common/IAsyncOperationStatusProvider.h"
#include "InsightsCore/Common/Stopwatch.h"
#include "InsightsCore/Filter/ViewModels/Filters.h"
#include "InsightsCore/Table/ViewModels/TableTreeNode.h"

#include <atomic>

#define UE_API TRACEINSIGHTSCORE_API

class FMenuBuilder;
class FUICommandList;

namespace UE::Insights
{

class FFilterConfigurator;
class FTable;
class FTableColumn;
class FTreeNodeGrouping;
class ITableCellValueSorter;

////////////////////////////////////////////////////////////////////////////////////////////////////

/** The filter collection - used for updating the list of tree nodes. */
typedef TFilterCollection<const FTableTreeNodePtr&> FTableTreeNodeFilterCollection;

/** The text based filter - used for updating the list of tree nodes. */
typedef TTextFilter<const FTableTreeNodePtr&> FTableTreeNodeTextFilter;

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class TRACEINSIGHTSCORE_API EAsyncOperationType : uint32
{
	NodeFiltering       = 1 << 0,
	Grouping            = 1 << 1,
	Aggregation         = 1 << 2,
	Sorting             = 1 << 3,
	HierarchyFiltering  = 1 << 4,
};

ENUM_CLASS_FLAGS(EAsyncOperationType);

struct FTableColumnConfig
{
	FName ColumnId;
	bool bIsVisible;
	float Width;
};

class STableTreeView;

class ITableTreeViewPreset
{
public:
	virtual FText GetName() const = 0;
	virtual FText GetToolTip() const = 0;
	virtual FName GetSortColumn() const = 0;
	virtual EColumnSortMode::Type GetSortMode() const = 0;
	virtual void SetCurrentGroupings(const TArray<TSharedPtr<FTreeNodeGrouping>>& InAvailableGroupings, TArray<TSharedPtr<FTreeNodeGrouping>>& InOutCurrentGroupings) const = 0;
	virtual void GetColumnConfigSet(TArray<FTableColumnConfig>& InOutConfigSet) const = 0;
	virtual void OnAppliedToView(STableTreeView& TableTreeView) const {}
};

class FTableTaskCancellationToken
{
public:
	FTableTaskCancellationToken()
		: bCancel(false)
	{}

	bool ShouldCancel() { return bCancel.load(); }
	void Cancel() { bCancel.store(true); }

private:
	std::atomic<bool> bCancel;
};

struct FTableTaskInfo
{
	FGraphEventRef Event;
	TSharedPtr<FTableTaskCancellationToken> CancellationToken;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * A custom widget used to display the list of tree nodes.
 */
class STableTreeView : public SCompoundWidget, public IAsyncOperationStatusProvider
{
	friend class FTableTreeViewNodeFilteringAsyncTask;
	friend class FTableTreeViewSortingAsyncTask;
	friend class FTableTreeViewGroupingAsyncTask;
	friend class FTableTreeViewHierarchyFilteringAsyncTask;
	friend class FTableTreeViewAsyncCompleteTask;
	friend class FSearchForItemToSelectTask;
	friend class FSelectNodeByTableRowIndexTask;

protected:
	enum class EGroupingMenuType
	{
		Change,
		Add
	};

public:
	/** Default constructor. */
	UE_API STableTreeView();

	/** Virtual destructor. */
	UE_API virtual ~STableTreeView();

	SLATE_BEGIN_ARGS(STableTreeView) {}
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 * @param InArgs - The declaration data for this widget
	 */
	UE_API void Construct(const FArguments& InArgs, TSharedPtr<FTable> InTablePtr);

	TSharedPtr<STreeView<FTableTreeNodePtr>> GetInnerTreeView() const { return TreeView; }

	TSharedPtr<FTable>& GetTable() { return Table; }
	const TSharedPtr<FTable>& GetTable() const { return Table; }

	UE_API virtual void Reset();

	UE_API void RebuildColumns();

	/**
	 * Ticks this widget.  Override in derived classes, but always call the parent implementation.
	 *
	 * @param  AllottedGeometry The space allotted for this widget
	 * @param  InCurrentTime  Current absolute real time
	 * @param  InDeltaTime  Real time passed since last tick
	 */
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/**
	 * Rebuilds the tree (if necessary).
	 * @param bResync - If true, it forces a resync even if the list did not changed since last sync.
	 */
	UE_API virtual void RebuildTree(bool bResync);

	UE_API FTableTreeNodePtr GetNodeByTableRowIndex(int32 RowIndex) const;
	UE_API void SelectNodeByTableRowIndex(int32 RowIndex);
	bool IsRunningAsyncUpdate() { return bIsUpdateRunning;  }

	UE_API void OnClose();

	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// IAsyncOperationStatusProvider implementation

	virtual bool IsRunning() const override { return bIsUpdateRunning; }

	UE_API virtual double GetAllOperationsDuration() override;
	virtual double GetCurrentOperationDuration() override { return 0.0; }
	virtual uint32 GetOperationCount() const override { return 1; }
	UE_API virtual FText GetCurrentOperationName() const override;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	/** Sets a log listing name to be used for any errors or warnings. Must be preregistered by the caller with the MessageLog module. */
	void SetLogListingName(const FName& InLogListingName) { LogListingName = InLogListingName; }
	const FName& GetLogListingName() { return LogListingName; }

	/** Gets the table row nodes. Each node corresponds to a table row. Index in this array corresponds to RowIndex in source table. */
	const TArray<FTableTreeNodePtr>& GetTableRowNodes() const { return TableRowNodes; }

	/** Gets the available groupings. */
	const TArray<TSharedPtr<FTreeNodeGrouping>>& GetAvailableGroupings() const { return AvailableGroupings; }

	/** Sets the current groupings. */
	UE_API void SetCurrentGroupings(TArray<TSharedPtr<FTreeNodeGrouping>>& InCurrentGroupings);

protected:
	UE_API void InitCommandList();

	UE_API virtual void ConstructWidget(TSharedPtr<FTable> InTablePtr);
	UE_API virtual TSharedPtr<SWidget> ConstructFilterToolbar();
	UE_API virtual TSharedRef<SWidget> ConstructHierarchyBreadcrumbTrail();
	virtual TSharedPtr<SWidget> ConstructToolbar() { return nullptr; }
	virtual TSharedPtr<SWidget> ConstructExtraToolbar() { return nullptr; }
	virtual TSharedPtr<SWidget> ConstructFooter() { return nullptr; }
	UE_API virtual void ConstructHeaderArea(TSharedRef<SVerticalBox> InHostBox);
	UE_API virtual void ConstructFooterArea(TSharedRef<SVerticalBox> InHostBox);

	UE_API void UpdateTree();

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Tree View - Context Menu

	UE_API TSharedPtr<SWidget> TreeView_GetMenuContent();
	UE_API void TreeView_BuildSortByMenu(FMenuBuilder& MenuBuilder);
	UE_API void TreeView_BuildViewColumnMenu(FMenuBuilder& MenuBuilder);
	UE_API void TreeView_BuildExportMenu(FMenuBuilder& MenuBuilder);

	UE_API bool ContextMenu_CopySelectedToClipboard_CanExecute() const;
	UE_API void ContextMenu_CopySelectedToClipboard_Execute();
	UE_API bool ContextMenu_CopyColumnToClipboard_CanExecute() const;
	UE_API void ContextMenu_CopyColumnToClipboard_Execute();
	UE_API bool ContextMenu_CopyColumnTooltipToClipboard_CanExecute() const;
	UE_API void ContextMenu_CopyColumnTooltipToClipboard_Execute();
	UE_API bool ContextMenu_ExpandSubtree_CanExecute() const;
	UE_API void ContextMenu_ExpandSubtree_Execute();
	UE_API bool ContextMenu_ExpandCriticalPath_CanExecute() const;
	UE_API void ContextMenu_ExpandCriticalPath_Execute();
	UE_API bool ContextMenu_CollapseSubtree_CanExecute() const;
	UE_API void ContextMenu_CollapseSubtree_Execute();
	UE_API bool ContextMenu_ExportToFile_CanExecute() const;
	UE_API void ContextMenu_ExportToFile_Execute(bool bInExportCollapsed, bool InExportLeafs);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Tree View - Columns' Header

	UE_API void InitializeAndShowHeaderColumns();

	UE_API FText GetColumnHeaderText(const FName ColumnId) const;

	UE_API TSharedRef<SWidget> TreeViewHeaderRow_GenerateColumnMenu(const FTableColumn& Column);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Tree View - Misc

	UE_API void TreeView_Refresh();

	/**
	 * Called by STreeView to retrieves the children for the specified parent item.
	 * @param InParent    - The parent node to retrieve the children from.
	 * @param OutChildren - List of children for the parent node.
	 */
	UE_API void TreeView_OnGetChildren(FTableTreeNodePtr InParent, TArray<FTableTreeNodePtr>& OutChildren);

	/** Called by STreeView when selection has changed. */
	UE_API virtual void TreeView_OnSelectionChanged(FTableTreeNodePtr SelectedItem, ESelectInfo::Type SelectInfo);

	/** Called by STreeView when a tree node is expanded or collapsed. */
	UE_API virtual void TreeView_OnExpansionChanged(FTableTreeNodePtr TreeNode, bool bShouldBeExpanded);

	/** Called by STreeView when a tree item is double clicked. */
	UE_API virtual void TreeView_OnMouseButtonDoubleClick(FTableTreeNodePtr TreeNode);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Tree View - Table Row

	/** Called by STreeView to generate a table row for the specified item. */
	UE_API TSharedRef<ITableRow> TreeView_OnGenerateRow(FTableTreeNodePtr TreeNode, const TSharedRef<STableViewBase>& OwnerTable);

	UE_API bool TableRow_ShouldBeEnabled(FTableTreeNodePtr NodePtr) const;

	UE_API void TableRow_SetHoveredCell(TSharedPtr<FTable> TablePtr, TSharedPtr<FTableColumn> ColumnPtr, FTableTreeNodePtr NodePtr);
	UE_API EHorizontalAlignment TableRow_GetColumnOutlineHAlignment(const FName ColumnId) const;

	UE_API FText TableRow_GetHighlightText() const;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Node Filtering (TableRowNodes --> FilteredNodes)

	UE_API void InitNodeFiltering();

	UE_API void OnNodeFilteringChanged();
	UE_API bool ScheduleNodeFilteringAsyncOperationIfNeeded();
	UE_API void ScheduleNodeFilteringAsyncOperation();
	UE_API FGraphEventRef StartNodeFilteringTask(FGraphEventRef Prerequisite = nullptr);

	UE_API void ApplyNodeFiltering();

	virtual bool HasCustomNodeFilter() const { return false; }
	virtual bool FilterNodeCustom(const FTableTreeNode& InNode) const { return true; }

	UE_API virtual bool FilterNode(const FFilterConfigurator& InFilterConfigurator, const FTableTreeNode& InNode) const;

	UE_API virtual void InitFilterConfigurator(FFilterConfigurator& InOutFilterConfigurator);
	UE_API virtual void UpdateFilterContext(const FFilterConfigurator& InFilterConfigurator, const FTableTreeNode& InNode) const;

	UE_API virtual TSharedRef<SWidget> ConstructFilterConfiguratorButton();
	UE_API FReply FilterConfigurator_OnClicked();

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Hierarchy Filtering (Root->Children hierarchy --> Root->FilteredChildren hierarchy)

	UE_API void InitHierarchyFiltering();

	/**
	 * Populates OutSearchStrings with the strings that should be used in searching.
	 *
	 * @param GroupOrStatNodePtr - the group and stat node to get a text description from.
	 * @param OutSearchStrings   - an array of strings to use in searching.
	 */
	static UE_API void HandleItemToStringArray(const FTableTreeNodePtr& GroupOrStatNodePtr, TArray<FString>& OutSearchStrings);

	UE_API void OnHierarchyFilteringChanged();
	UE_API bool ScheduleHierarchyFilteringAsyncOperationIfNeeded();
	UE_API void ScheduleHierarchyFilteringAsyncOperation();
	UE_API FGraphEventRef StartHierarchyFilteringTask(FGraphEventRef Prerequisite = nullptr);

	UE_API void ApplyHierarchyFiltering();
	UE_API void ApplyEmptyHierarchyFilteringRec(FTableTreeNodePtr NodePtr);
	UE_API bool ApplyHierarchyFilteringRec(FTableTreeNodePtr NodePtr);

	/** Set all the nodes belonging to a subtree as visible. Returns true if the caller node should be expanded. */
	UE_API bool MakeSubtreeVisible(FTableTreeNodePtr NodePtr, bool bFilterIsEmpty);

	UE_API virtual TSharedRef<SWidget> ConstructSearchBox();
	UE_API void SearchBox_OnTextChanged(const FText& InFilterText);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Grouping (FilteredNodes --> Root->Children hierarchy)

	UE_API void CreateGroupings();
	UE_API virtual void InternalCreateGroupings();

	UE_API void OnGroupingChanged();
	UE_API bool ScheduleGroupingAsyncOperationIfNeeded();
	UE_API void ScheduleGroupingAsyncOperation();
	UE_API FGraphEventRef StartGroupingTask(FGraphEventRef Prerequisite = nullptr);
	UE_API void ApplyGrouping();
	UE_API void CreateGroups(const TArray<TSharedPtr<FTreeNodeGrouping>>& Groupings);
	UE_API void GroupNodesRec(const TArray<FTableTreeNodePtr>& Nodes, FTableTreeNode& ParentGroup, int32 GroupingDepth, const TArray<TSharedPtr<FTreeNodeGrouping>>& Groupings);

	UE_API void RebuildGroupingCrumbs();
	UE_API TSharedRef<SWidget> GetGroupingCrumbButtonContent(const TSharedPtr<FTreeNodeGrouping>& CrumbGrouping, const FTextBlockStyle* InTextStyle);
	UE_API void OnGroupingCrumbClicked(const TSharedPtr<FTreeNodeGrouping>& InEntry);
	UE_API TSharedRef<SWidget> CreateGroupingMenuWidget(const FTreeNodeGrouping& Grouping) const;
	UE_API void AddGroupingMenuEntries(FMenuBuilder& MenuBuilder, EGroupingMenuType MenuType, const TSharedPtr<FTreeNodeGrouping> CrumbGrouping, TFunction<bool(const FTreeNodeGrouping&)> FilterFunc);
	UE_API void BuildGroupingSubMenu(FMenuBuilder& MenuBuilder, EGroupingMenuType MenuType, const TSharedPtr<FTreeNodeGrouping> CrumbGrouping);
	UE_API TSharedRef<SWidget> GetGroupingCrumbMenuContent(const TSharedPtr<FTreeNodeGrouping>& CrumbGrouping);

	UE_API void PreChangeGroupings();
	UE_API void PostChangeGroupings();
	UE_API int32 GetGroupingDepth(const TSharedPtr<FTreeNodeGrouping>& Grouping) const;

	UE_API void GroupingCrumbMenu_Reset_Execute();
	UE_API void GroupingCrumbMenu_Remove_Execute(const TSharedPtr<FTreeNodeGrouping> Grouping);
	UE_API void GroupingCrumbMenu_MoveLeft_Execute(const TSharedPtr<FTreeNodeGrouping> Grouping);
	UE_API void GroupingCrumbMenu_MoveRight_Execute(const TSharedPtr<FTreeNodeGrouping> Grouping);
	UE_API void GroupingCrumbMenu_Change_Execute(const TSharedPtr<FTreeNodeGrouping> OldGrouping, const TSharedPtr<FTreeNodeGrouping> NewGrouping);
	UE_API bool GroupingCrumbMenu_Change_CanExecute(const TSharedPtr<FTreeNodeGrouping> OldGrouping, const TSharedPtr<FTreeNodeGrouping> NewGrouping) const;
	UE_API void GroupingCrumbMenu_Add_Execute(const TSharedPtr<FTreeNodeGrouping> Grouping, const TSharedPtr<FTreeNodeGrouping> AfterGrouping);
	UE_API bool GroupingCrumbMenu_Add_CanExecute(const TSharedPtr<FTreeNodeGrouping> Grouping, const TSharedPtr<FTreeNodeGrouping> AfterGrouping) const;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Aggregation (Root->Children hierarchy)

	static UE_API void UpdateCStringSameValueAggregationSingleNode(const FTableColumn& InColumn, FTableTreeNode& GroupNode);
	static UE_API void UpdateCStringSameValueAggregationRec(const FTableColumn& InColumn, FTableTreeNode& GroupNode);

	template<typename T, bool bSetInitialValue, bool bIsRercursive>
	static void UpdateAggregation(const FTableColumn& InColumn, FTableTreeNode& InOutGroupNode, const T InitialAggregatedValue, TFunctionRef<T(T, const FTableCellValue&)> ValueGetterFunc);

	template<bool bIsRercursive>
	static void UpdateAggregatedValues(TSharedPtr<FTable> InTable, FTableTreeNode& InOutGroupNode);

	UE_API void UpdateAggregatedValuesSingleNode(FTableTreeNode& GroupNode);
	UE_API void UpdateAggregatedValuesRec(FTableTreeNode& GroupNode);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Sorting (Root->Children hierarchy)

	static UE_API const EColumnSortMode::Type GetDefaultColumnSortMode();
	static UE_API const FName GetDefaultColumnBeingSorted();

	UE_API void CreateSortings();
	UE_API void UpdateCurrentSortingByColumn();

	UE_API void OnSortingChanged();
	UE_API bool ScheduleSortingAsyncOperationIfNeeded();
	UE_API void ScheduleSortingAsyncOperation();
	UE_API FGraphEventRef StartSortingTask(FGraphEventRef Prerequisite = nullptr);
	UE_API void ApplySorting();
	UE_API void SortTreeNodes(ITableCellValueSorter* InSorter, EColumnSortMode::Type InColumnSortMode);
	UE_API void SortTreeNodesRec(FTableTreeNode& GroupNode, const ITableCellValueSorter& Sorter, EColumnSortMode::Type InColumnSortMode);

	UE_API EColumnSortMode::Type GetSortModeForColumn(const FName ColumnId) const;
	UE_API void SetSortModeForColumn(const FName& ColumnId, EColumnSortMode::Type SortMode);
	UE_API void OnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type SortMode);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Sorting actions

	// SortMode (HeaderMenu)
	UE_API bool HeaderMenu_SortMode_IsChecked(const FName ColumnId, const EColumnSortMode::Type InSortMode);
	UE_API bool HeaderMenu_SortMode_CanExecute(const FName ColumnId, const EColumnSortMode::Type InSortMode) const;
	UE_API void HeaderMenu_SortMode_Execute(const FName ColumnId, const EColumnSortMode::Type InSortMode);

	// SortMode (ContextMenu)
	UE_API bool ContextMenu_SortMode_IsChecked(const EColumnSortMode::Type InSortMode);
	UE_API bool ContextMenu_SortMode_CanExecute(const EColumnSortMode::Type InSortMode) const;
	UE_API void ContextMenu_SortMode_Execute(const EColumnSortMode::Type InSortMode);

	// SortByColumn (ContextMenu)
	UE_API bool ContextMenu_SortByColumn_IsChecked(const FName ColumnId);
	UE_API bool ContextMenu_SortByColumn_CanExecute(const FName ColumnId) const;
	UE_API void ContextMenu_SortByColumn_Execute(const FName ColumnId);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Column visibility actions

	// ShowColumn
	UE_API bool CanShowColumn(const FName ColumnId) const;
	UE_API void ShowColumn(const FName ColumnId);
	UE_API void ShowColumn(FTableColumn& Column);

	// HideColumn
	UE_API bool CanHideColumn(const FName ColumnId) const;
	UE_API void HideColumn(const FName ColumnId);
	UE_API void HideColumn(FTableColumn& Column);

	// ToggleColumnVisibility
	UE_API bool IsColumnVisible(const FName ColumnId);
	UE_API bool CanToggleColumnVisibility(const FName ColumnId) const;
	UE_API void ToggleColumnVisibility(const FName ColumnId);

	// ShowAllColumns (ContextMenu)
	UE_API bool ContextMenu_ShowAllColumns_CanExecute() const;
	UE_API void ContextMenu_ShowAllColumns_Execute();

	// ResetColumns (ContextMenu)
	UE_API bool ContextMenu_ResetColumns_CanExecute() const;
	UE_API void ContextMenu_ResetColumns_Execute();

	// HideAllColumns (ContextMenu)
	UE_API bool ContextMenu_HideAllColumns_CanExecute() const;
	UE_API void ContextMenu_HideAllColumns_Execute();

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Async

	UE_API virtual void OnPreAsyncUpdate();
	UE_API virtual void OnPostAsyncUpdate();

	void AddPendingAsyncOperation(EAsyncOperationType InType) { EnumAddFlags(PendingAsyncOperations, InType); }
	bool HasPendingAsyncOperation(EAsyncOperationType InType) const { return EnumHasAnyFlags(PendingAsyncOperations, InType); }
	void ClearPendingAsyncOperation(EAsyncOperationType InType) { EnumRemoveFlags(PendingAsyncOperations, InType); }
	void ClearPendingAsyncOperations() { PendingAsyncOperations = static_cast<EAsyncOperationType>(0); }

	void AddInProgressAsyncOperation(EAsyncOperationType InType) { EnumAddFlags(InProgressAsyncOperations, InType); }
	bool HasInProgressAsyncOperation(EAsyncOperationType InType) const { return EnumHasAnyFlags(InProgressAsyncOperations, InType); }
	void ClearInProgressAsyncOperations() { InProgressAsyncOperations = static_cast<EAsyncOperationType>(0); }

	UE_API void StartPendingAsyncOperations();

	UE_API void CancelCurrentAsyncOp();

	////////////////////////////////////////////////////////////////////////////////////////////////////

	UE_API void CountNumNodesPerDepthRec(FBaseTreeNode* InRoot, TArray<int32>& InOutNumNodesPerDepth, int32 InDepth, int32 InMaxDepth, int32 InMaxNodes) const;
	UE_API void ApplyNodeExpansion(const FTableTreeNodePtr& InNode);
	UE_API void SetExpandValueForChildGroups(FBaseTreeNode* InRoot, int32 InMaxExpandedNodes, int32 MaxDepthToExpand, bool InValue);
	UE_API void SetExpandValueForChildGroupsRec(FBaseTreeNode* InRoot, int32 InDepth, int32 InMaxDepth, bool InValue);

	virtual void ExtendMenu(TSharedRef<FExtender> Extender) {}
	virtual void ExtendMenu(FMenuBuilder& Menu) {}

	typedef TFunctionRef<void(TArray<FBaseTreeNodePtr>& InNodes)> WriteToFileCallback;
	UE_API void ExportToFileRec(const FBaseTreeNodePtr& InGroupNode, TArray<FBaseTreeNodePtr>& InNodes, bool bInExportCollapsed, bool InExportLeafs, WriteToFileCallback Callback);

	FText GetTreeViewBannerText() const { return TreeViewBannerText; }
	UE_API virtual void UpdateBannerText();

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Presets

	virtual void InitAvailableViewPresets() {}
	const TArray<TSharedRef<ITableTreeViewPreset>>* GetAvailableViewPresets() const { return &AvailableViewPresets; }
	UE_API TSharedPtr<ITableTreeViewPreset> GetSelectedViewPreset() const;
	UE_API void SelectViewPreset(TSharedPtr<ITableTreeViewPreset> InPreset);
	UE_API void ApplyViewPreset(const ITableTreeViewPreset& InPreset);
	UE_API void ApplyColumnConfig(const TArrayView<FTableColumnConfig>& InTableConfig);
	UE_API void ViewPreset_OnSelectionChanged(TSharedPtr<ITableTreeViewPreset> InPreset, ESelectInfo::Type SelectInfo);
	UE_API TSharedRef<SWidget> ViewPreset_OnGenerateWidget(TSharedRef<ITableTreeViewPreset> InPreset);
	UE_API FText ViewPreset_GetSelectedText() const;
	UE_API FText ViewPreset_GetSelectedToolTipText() const;
	UE_API void ConstructViewPreset(TSharedPtr<SHorizontalBox> InHorizontalBox, float InMinDesiredWidth = 50.0f);

	////////////////////////////////////////////////////////////////////////////////////////////////////

	virtual void SearchForItem(TSharedPtr<FTableTaskCancellationToken> CancellationToken) {};

	// Table data tasks should be tasks that operate read only operations on the data from the Table
	// They should not operate on the tree nodes because they will run concurrently with the populated table UI.
	template<typename T, typename... TArgs>
	TSharedPtr<FTableTaskInfo> StartTableDataTask(TArgs&&... Args)
	{
		TSharedPtr<FTableTaskInfo> Info = MakeShared<FTableTaskInfo>();
		Info->CancellationToken = MakeShared<FTableTaskCancellationToken>();
		Info->Event = TGraphTask<T>::CreateTask().ConstructAndDispatchWhenReady(Info->CancellationToken, Forward<TArgs>(Args)...);
		DataTaskInfos.Add(Info);
		return Info;
	}

	UE_API void StopAllTableDataTasks(bool bWait = true);

protected:
	/** Table view model. */
	TSharedPtr<FTable> Table;

	TSharedPtr<FUICommandList> CommandList;

	//////////////////////////////////////////////////
	// Widget

	/** The child STreeView widget. */
	TSharedPtr<STreeView<FTableTreeNodePtr>> TreeView;

	/** Holds the tree view header row widget which display all columns in the tree view. */
	TSharedPtr<SHeaderRow> TreeViewHeaderRow;

	/** External scrollbar used to synchronize tree view position. */
	TSharedPtr<SScrollBar> ExternalScrollbar;

	//////////////////////////////////////////////////
	// Hovered Column, Hovered Tree Node

	/** Name of the column currently being hovered by the mouse. */
	FName HoveredColumnId;

	/** A shared pointer to the tree node currently being hovered by the mouse. */
	FTableTreeNodePtr HoveredNodePtr;

	//////////////////////////////////////////////////
	// Tree Nodes

	/** The root node of the tree. It is invisible; only its children are displayed in the tree view. */
	FTableTreeNodePtr Root;

	/** Table row nodes. Each node corresponds to a table row. Index in this array corresponds to RowIndex in source table. */
	TArray<FTableTreeNodePtr> TableRowNodes;

	/** Filtered table nodes. These are the nodes from the TableRowNodes array, after applying the node filtering. */
	TArray<FTableTreeNodePtr> FilteredNodes;

	/** A pointer to the filtered table nodes. If node filtering is empty, this points directly to TableRowNodes. */
	TArray<FTableTreeNodePtr>* FilteredNodesPtr = nullptr;

	/** A filtered array of group nodes to be displayed in the tree widget. */
	TArray<FTableTreeNodePtr> FilteredGroupNodes;

	int32 MaxNodesToAutoExpand = 1000;
	int32 MaxDepthToAutoExpand = 4;
	int32 MaxNodesToExpand = 1000000;
	int32 MaxDepthToExpand = 100;

	//////////////////////////////////////////////////
	// Search box & the hierarchy filtering

	/** The search box widget used to filter items displayed in the stats and groups tree. */
	TSharedPtr<SSearchBox> SearchBox;

	/** The filter collection. */
	TSharedPtr<FTableTreeNodeFilterCollection> Filters;

	/** The text based filter. */
	TSharedPtr<FTableTreeNodeTextFilter> TextFilter;

	/** The text based filter actually used in the Hierarchy Filtering async task. */
	TSharedPtr<FTableTreeNodeTextFilter> CurrentAsyncOpTextFilter;

	//////////////////////////////////////////////////
	// Node filtering

	TSharedPtr<FFilterConfigurator> FilterConfigurator;
	FDelegateHandle OnFilterChangesCommittedHandle;

	/** The filter configurator actually used in the Hierarchy Filtering async task. */
	FFilterConfigurator* CurrentAsyncOpFilterConfigurator = nullptr;

	mutable FFilterContext FilterContext;

	//////////////////////////////////////////////////
	// Grouping

	TArray<TSharedPtr<FTreeNodeGrouping>> AvailableGroupings;

	/** How we group the tree nodes? */
	TArray<TSharedPtr<FTreeNodeGrouping>> CurrentGroupings;

	TSharedPtr<SBreadcrumbTrail<TSharedPtr<FTreeNodeGrouping>>> GroupingBreadcrumbTrail;

	/** The groupings actually used in the Grouping async task. */
	TArray<TSharedPtr<FTreeNodeGrouping>> CurrentAsyncOpGroupings;

	/** The columns hidden when grouping is applied. */
	TSet<FName> GroupingHiddenColumns;

	//////////////////////////////////////////////////
	// Sorting

	/** All available sorters. */
	TArray<TSharedPtr<ITableCellValueSorter>> AvailableSorters;

	/** Current sorter. It is nullptr if sorting is disabled. */
	TSharedPtr<ITableCellValueSorter> CurrentSorter;

	/** Name of the column currently being sorted. Can be NAME_None if sorting is disabled (CurrentSorting == nullptr) or if a complex sorting is used (CurrentSorting != nullptr). */
	FName ColumnBeingSorted;

	/** How we sort the nodes? Ascending or Descending. */
	EColumnSortMode::Type ColumnSortMode = EColumnSortMode::None;

	/** The sorter actually used in the Sorting async task. */
	ITableCellValueSorter* CurrentAsyncOpSorter = nullptr;

	/** The sort mode actually used in the Sorting async task. */
	EColumnSortMode::Type CurrentAsyncOpColumnSortMode;

	//////////////////////////////////////////////////
	// Async Operations

	bool bRunInAsyncMode = false;
	bool bIsUpdateRunning = false;
	bool bIsCloseScheduled = false;

	TArray<FTableTreeNodePtr> DummyGroupNodes;
	FGraphEventRef InProgressAsyncOperationEvent;
	FGraphEventRef AsyncCompleteTaskEvent;
	EAsyncOperationType PendingAsyncOperations = static_cast<EAsyncOperationType>(0);
	EAsyncOperationType InProgressAsyncOperations = static_cast<EAsyncOperationType>(0);
	TSharedPtr<class SAsyncOperationStatus> AsyncOperationStatus;
	FStopwatch AsyncUpdateStopwatch;
	FAsyncOperationProgress AsyncOperationProgress;
	FGraphEventRef DispatchEvent;
	TArray<TSharedPtr<FTableTaskInfo>> DataTaskInfos;

	//////////////////////////////////////////////////

	FText TreeViewBannerText;

	TArray<TSharedRef<ITableTreeViewPreset>> AvailableViewPresets;
	TSharedPtr<ITableTreeViewPreset> SelectedViewPreset;
	TSharedPtr<SComboBox<TSharedRef<ITableTreeViewPreset>>> PresetComboBox;

	//////////////////////////////////////////////////
	// Logging

	/** A log listing name to be used for any errors or warnings. */
	FName LogListingName;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTableTreeViewNodeFilteringAsyncTask
{
public:
	FTableTreeViewNodeFilteringAsyncTask(STableTreeView* InPtr)
		: TableTreeViewPtr(InPtr)
	{
	}

	FORCEINLINE TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(FTableTreeViewNodeFilteringAsyncTask, STATGROUP_TaskGraphTasks); }
	ENamedThreads::Type GetDesiredThread() { return ENamedThreads::Type::AnyThread; }
	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		if (TableTreeViewPtr)
		{
			TableTreeViewPtr->ApplyNodeFiltering();
		}
	}

private:
	STableTreeView* TableTreeViewPtr = nullptr;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTableTreeViewHierarchyFilteringAsyncTask
{
public:
	FTableTreeViewHierarchyFilteringAsyncTask(STableTreeView* InPtr)
		: TableTreeViewPtr(InPtr)
	{
	}

	FORCEINLINE TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(FTableTreeViewHierarchyFilteringAsyncTask, STATGROUP_TaskGraphTasks); }
	ENamedThreads::Type GetDesiredThread() { return ENamedThreads::Type::AnyThread; }
	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		if (TableTreeViewPtr)
		{
			TableTreeViewPtr->ApplyHierarchyFiltering();
		}
	}

private:
	STableTreeView* TableTreeViewPtr = nullptr;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTableTreeViewSortingAsyncTask
{
public:
	FTableTreeViewSortingAsyncTask(STableTreeView* InPtr, ITableCellValueSorter* InSorter, EColumnSortMode::Type InColumnSortMode)
		: TableTreeViewPtr(InPtr)
		, Sorter(InSorter)
		, ColumnSortMode(InColumnSortMode)
	{
	}

	FORCEINLINE TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(FTableTreeViewSortingAsyncTask, STATGROUP_TaskGraphTasks); }
	ENamedThreads::Type GetDesiredThread() { return ENamedThreads::Type::AnyThread; }
	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		if (TableTreeViewPtr)
		{
			TableTreeViewPtr->SortTreeNodes(Sorter, ColumnSortMode);
		}
	}

private:
	STableTreeView* TableTreeViewPtr = nullptr;
	ITableCellValueSorter* Sorter = nullptr;
	EColumnSortMode::Type ColumnSortMode = EColumnSortMode::Type::None;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTableTreeViewGroupingAsyncTask
{
public:
	FTableTreeViewGroupingAsyncTask(STableTreeView* InPtr, TArray<TSharedPtr<FTreeNodeGrouping>>* InGroupings)
		: TableTreeViewPtr(InPtr)
		, Groupings(InGroupings)
	{
	}

	FORCEINLINE TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(FTableTreeViewGroupingAsyncTask, STATGROUP_TaskGraphTasks); }
	ENamedThreads::Type GetDesiredThread() { return ENamedThreads::Type::AnyThread; }
	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		if (TableTreeViewPtr)
		{
			TableTreeViewPtr->CreateGroups(*Groupings);
		}
	}

private:
	STableTreeView* TableTreeViewPtr = nullptr;
	TArray<TSharedPtr<FTreeNodeGrouping>>* Groupings = nullptr;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FSearchForItemToSelectTask
{
public:
	FSearchForItemToSelectTask(TSharedPtr<FTableTaskCancellationToken> InToken, TSharedPtr<STableTreeView> InPtr)
		: CancellationToken(InToken)
		, TableTreeViewPtr(InPtr)
	{
	}

	FORCEINLINE TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(FSearchForItemToSelectTask, STATGROUP_TaskGraphTasks); }
	ENamedThreads::Type GetDesiredThread() { return ENamedThreads::Type::AnyThread; }
	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		if (!CancellationToken.IsValid() || !CancellationToken->ShouldCancel())
		{
			if (TableTreeViewPtr.IsValid())
			{
				TableTreeViewPtr->SearchForItem(CancellationToken);
			}
		}
	}

private:
	TSharedPtr<FTableTaskCancellationToken> CancellationToken;
	TSharedPtr<STableTreeView> TableTreeViewPtr;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FSelectNodeByTableRowIndexTask
{
public:
	FSelectNodeByTableRowIndexTask(TSharedPtr<FTableTaskCancellationToken> InToken, TSharedPtr<STableTreeView> InPtr, uint32 InRowIndex)
		: CancellationToken(InToken)
		, TableTreeViewPtr(InPtr)
		, RowIndex(InRowIndex)
	{
	}

	FORCEINLINE TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(FSelectNodeByTableRowIndexTask, STATGROUP_TaskGraphTasks); }
	ENamedThreads::Type GetDesiredThread() { return ENamedThreads::Type::GameThread; }
	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		if (!CancellationToken.IsValid() || !CancellationToken->ShouldCancel())
		{
			if (TableTreeViewPtr.IsValid())
			{
				TableTreeViewPtr->SelectNodeByTableRowIndex(RowIndex);
			}
		}
	}

private:
	TSharedPtr< FTableTaskCancellationToken> CancellationToken;
	TSharedPtr<STableTreeView> TableTreeViewPtr;
	uint32 RowIndex = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef UE_API
