// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#include "InsightsCore/Table/ViewModels/TableTreeNode.h"
#include "InsightsCore/Table/Widgets/STableTreeViewRow.h"

#define UE_API TRACEINSIGHTSCORE_API

class ITableRow;
class IToolTip;

namespace UE::Insights
{

class FTable;
class FTableColumn;

DECLARE_DELEGATE_ThreeParams(FSetHoveredTableTreeViewCell, TSharedPtr<FTable> /*TablePtr*/, TSharedPtr<FTableColumn> /*ColumnPtr*/, FTableTreeNodePtr /*TableTreeNodePtr*/);

class STableTreeViewCell : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STableTreeViewCell) {}
		SLATE_EVENT(FSetHoveredTableTreeViewCell, OnSetHoveredCell)
		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_ARGUMENT(TSharedPtr<FTable>, TablePtr)
		SLATE_ARGUMENT(TSharedPtr<FTableColumn>, ColumnPtr)
		SLATE_ARGUMENT(FTableTreeNodePtr, TableTreeNodePtr)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, const TSharedRef<ITableRow>& InTableRow);

protected:
	UE_API TSharedRef<SWidget> GenerateWidgetForColumn(const FArguments& InArgs);
	UE_API TSharedRef<SWidget> GenerateWidgetForNameColumn(const FArguments& InArgs);
	UE_API TSharedRef<SWidget> GenerateWidgetForTableColumn(const FArguments& InArgs);

	/**
	 * The system will use this event to notify a widget that the cursor has entered it. This event is NOT bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 */
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);
		SetHoveredCellDelegate.ExecuteIfBound(TablePtr, ColumnPtr, TableTreeNodePtr);
	}

	/**
	 * The system will use this event to notify a widget that the cursor has left it. This event is NOT bubbled.
	 *
	 * @param MouseEvent Information about the input event
	 */
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override
	{
		SCompoundWidget::OnMouseLeave(MouseEvent);
		SetHoveredCellDelegate.ExecuteIfBound(nullptr, nullptr, nullptr);
	}

	/**
	 * Called during drag and drop when the drag enters a widget.
	 *
	 * Enter/Leave events in slate are meant as lightweight notifications.
	 * So we do not want to capture mouse or set focus in response to these.
	 * However, OnDragEnter must also support external APIs (e.g. OLE Drag/Drop)
	 * Those require that we let them know whether we can handle the content
	 * being dragged OnDragEnter.
	 *
	 * The concession is to return a can_handled/cannot_handle
	 * boolean rather than a full FReply.
	 *
	 * @param MyGeometry      The geometry of the widget receiving the event.
	 * @param DragDropEvent   The drag and drop event.
	 *
	 * @return A reply that indicated whether the contents of the DragDropEvent can potentially be processed by this widget.
	 */
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
	{
		SCompoundWidget::OnDragEnter(MyGeometry, DragDropEvent);
		SetHoveredCellDelegate.ExecuteIfBound(TablePtr, ColumnPtr, TableTreeNodePtr);
	}

	/**
	 * Called during drag and drop when the drag leaves a widget.
	 *
	 * @param DragDropEvent   The drag and drop event.
	 */
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent)  override
	{
		SCompoundWidget::OnDragLeave(DragDropEvent);
		SetHoveredCellDelegate.ExecuteIfBound(nullptr, nullptr, nullptr);
	}

	EVisibility GetHintIconVisibility() const
	{
		return IsHovered() ? EVisibility::Visible : EVisibility::Hidden;
	}

	UE_API bool IsSelected() const;

	const FSlateBrush* GetIcon() const
	{
		return TableTreeNodePtr->GetIcon();
	}

	FText GetDisplayName() const
	{
		return TableTreeNodePtr->GetDisplayName();
	}

	FText GetExtraDisplayName() const
	{
		return TableTreeNodePtr->GetExtraDisplayName();
	}

	EVisibility HasExtraDisplayName() const
	{
		return TableTreeNodePtr->HasExtraDisplayName() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	UE_API FText GetValueAsText() const;

	UE_API FSlateColor GetIconColorAndOpacity() const;
	UE_API FSlateColor GetDisplayNameColorAndOpacity() const;
	UE_API FSlateColor GetExtraDisplayNameColorAndOpacity() const;
	UE_API FSlateColor GetNormalTextColorAndOpacity() const;
	UE_API FLinearColor GetShadowColorAndOpacity() const;

protected:
	/** A weak pointer to the parent table row widget. */
	TWeakPtr<ITableRow> WeakTableRow;

	/** A shared pointer to the table view model. */
	TSharedPtr<FTable> TablePtr;

	/** A shared pointer to the table column view model. */
	TSharedPtr<FTableColumn> ColumnPtr;

	/** A shared pointer to the tree node. */
	FTableTreeNodePtr TableTreeNodePtr;

	FSetHoveredTableTreeViewCell SetHoveredCellDelegate;
};

} // namespace UE::Insights

#undef UE_API
