// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

#include "InsightsCore/Table/ViewModels/TableTreeNode.h"

#define UE_API TRACEINSIGHTSCORE_API

class IToolTip;

namespace UE::Insights
{

class FTable;
class FTableColumn;
class STableTreeRowToolTip;

DECLARE_DELEGATE_RetVal_OneParam(bool, FTableTreeNodeShouldBeEnabledDelegate, FTableTreeNodePtr /*NodePtr*/);
DECLARE_DELEGATE_RetVal_OneParam(bool, FIsColumnVisibleDelegate, const FName /*ColumnId*/);
DECLARE_DELEGATE_RetVal_OneParam(EHorizontalAlignment, FGetColumnOutlineHAlignmentDelegate, const FName /*ColumnId*/);
DECLARE_DELEGATE_ThreeParams(FSetHoveredTableTreeViewCell, TSharedPtr<FTable> /*TablePtr*/, TSharedPtr<FTableColumn> /*ColumnPtr*/, FTableTreeNodePtr /*TableTreeNodePtr*/);

/** Widget that represents a table row in the tree control. Generates widgets for each column on demand. */
class STableTreeViewRow : public SMultiColumnTableRow<FTableTreeNodePtr>
{
public:
	SLATE_BEGIN_ARGS(STableTreeViewRow) {}
		SLATE_EVENT(FTableTreeNodeShouldBeEnabledDelegate, OnShouldBeEnabled)
		SLATE_EVENT(FIsColumnVisibleDelegate, OnIsColumnVisible)
		SLATE_EVENT(FGetColumnOutlineHAlignmentDelegate, OnGetColumnOutlineHAlignmentDelegate)
		SLATE_EVENT(FSetHoveredTableTreeViewCell, OnSetHoveredCell)
		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_ARGUMENT(TSharedPtr<FTable>, TablePtr)
		SLATE_ARGUMENT(FTableTreeNodePtr, TableTreeNodePtr)
	SLATE_END_ARGS()

public:
	UE_API void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

	UE_API virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnId) override;

	/**
	 * Called when Slate detects that a widget started to be dragged.
	 * Usage:
	 * A widget can ask Slate to detect a drag.
	 * OnMouseDown() reply with FReply::Handled().DetectDrag(SharedThis(this)).
	 * Slate will either send an OnDragDetected() event or do nothing.
	 * If the user releases a mouse button or leaves the widget before
	 * a drag is triggered (maybe user started at the very edge) then no event will be
	 * sent.
	 *
	 * @param  InMyGeometry  Widget geometry
	 * @param  InMouseEvent  MouseMove that triggered the drag
	 *
	 */
	UE_API virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	UE_API TSharedRef<IToolTip> GetRowToolTip() const;
	UE_API void InvalidateContent();

protected:
	UE_API TSharedRef<SWidget> CreateCellWidget(FName ColumnId);
	UE_API FSlateColor GetBackgroundColorAndOpacity() const;
	UE_API FSlateColor GetBackgroundColorAndOpacity(double Time) const;
	UE_API FSlateColor GetOutlineColorAndOpacity() const;
	UE_API const FSlateBrush* GetOutlineBrush(const FName ColumnId) const;
	UE_API bool HandleShouldBeEnabled() const;
	UE_API EVisibility IsColumnVisible(const FName ColumnId) const;
	UE_API void OnSetHoveredCell(TSharedPtr<FTable> InTablePtr, TSharedPtr<FTableColumn> InColumnPtr, FTableTreeNodePtr InTreeNodePtr);

protected:
	/** A shared pointer to the table view model. */
	TSharedPtr<FTable> TablePtr;

	/** Data context for this table row. */
	FTableTreeNodePtr TableTreeNodePtr;

	FTableTreeNodeShouldBeEnabledDelegate OnShouldBeEnabled;
	FIsColumnVisibleDelegate IsColumnVisibleDelegate;
	FSetHoveredTableTreeViewCell SetHoveredCellDelegate;
	FGetColumnOutlineHAlignmentDelegate GetColumnOutlineHAlignmentDelegate;

	/** Text to be highlighted on tree node's name. */
	TAttribute<FText> HighlightText;

	TSharedPtr<STableTreeRowToolTip> RowToolTip;
};

} // namespace UE::Insights

#undef UE_API
