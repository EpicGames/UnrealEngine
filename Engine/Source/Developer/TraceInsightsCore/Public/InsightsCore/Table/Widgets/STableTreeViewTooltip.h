// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/IToolTip.h"
#include "Widgets/SToolTip.h"

#define UE_API TRACEINSIGHTSCORE_API

class SGridPanel;

namespace UE::Insights
{

class FTable;
class FTableColumn;
class FTableTreeNode;

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Tooltip for STableTreeView widget. */
class STableTreeViewTooltip
{
public:
	STableTreeViewTooltip() = delete;

	static UE_API TSharedPtr<SToolTip> GetTableTooltip(const FTable& Table);
	static UE_API TSharedPtr<SToolTip> GetColumnTooltip(const FTableColumn& Column);
	static UE_API TSharedPtr<SToolTip> GetRowTooltip(const TSharedPtr<FTableTreeNode> TreeNodePtr);

private:
	static UE_API void AddGridRow(TSharedPtr<SGridPanel> Grid, int32& Row, const FText& Name, const FText& Value);
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class STableTreeRowToolTip : public IToolTip
{
public:
	STableTreeRowToolTip(const TSharedPtr<FTableTreeNode> InTreeNodePtr) : TreeNodePtr(InTreeNodePtr) {}
	virtual ~STableTreeRowToolTip() = default;

	virtual TSharedRef<SWidget> AsWidget()
	{
		CreateToolTipWidget();
		return ToolTipWidget.ToSharedRef();
	}

	virtual TSharedRef<SWidget> GetContentWidget()
	{
		CreateToolTipWidget();
		return ToolTipWidget->GetContentWidget();
	}

	virtual void SetContentWidget(const TSharedRef<SWidget>& InContentWidget)
	{
		CreateToolTipWidget();
		ToolTipWidget->SetContentWidget(InContentWidget);
	}

	void InvalidateWidget()
	{
		ToolTipWidget.Reset();
	}

	virtual bool IsEmpty() const { return false; }
	virtual bool IsInteractive() const { return false; }
	virtual void OnOpening() {}
	virtual void OnClosed() {}

private:
	void CreateToolTipWidget()
	{
		if (!ToolTipWidget.IsValid())
		{
			ToolTipWidget = STableTreeViewTooltip::GetRowTooltip(TreeNodePtr);
			check(ToolTipWidget.IsValid());
		}
	}

private:
	TSharedPtr<SToolTip> ToolTipWidget;
	const TSharedPtr<FTableTreeNode> TreeNodePtr;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef UE_API
