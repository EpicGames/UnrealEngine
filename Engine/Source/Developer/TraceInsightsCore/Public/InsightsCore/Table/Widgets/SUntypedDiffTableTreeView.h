// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

// TraceInsights
#include "InsightsCore/Table/ViewModels/UntypedTable.h"
#include "InsightsCore/Table/Widgets/SUntypedTableTreeView.h"

#define UE_API TRACEINSIGHTSCORE_API

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class SUntypedDiffTableTreeView : public SUntypedTableTreeView
{
public:
	UE_API void UpdateSourceTableA(const FString& Name, TSharedPtr<TraceServices::IUntypedTable> SourceTable);
	UE_API void UpdateSourceTableB(const FString& Name, TSharedPtr<TraceServices::IUntypedTable> SourceTable);

protected:
	UE_API FReply SwapTables_OnClicked();
	UE_API FText GetSwapButtonText() const;

	UE_API virtual TSharedPtr<SWidget> ConstructToolbar() override;

	UE_API void RequestMergeTables();

private:
	TSharedPtr<TraceServices::IUntypedTable> SourceTableA;
	TSharedPtr<TraceServices::IUntypedTable> SourceTableB;
	FString TableNameA;
	FString TableNameB;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef UE_API
