// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

// TraceInsightsCore
#include "InsightsCore/Common/Stopwatch.h"
#include "InsightsCore/Table/ViewModels/UntypedTable.h"
#include "InsightsCore/Table/Widgets/STableTreeView.h"

#define UE_API TRACEINSIGHTSCORE_API

namespace TraceServices
{
	class IUntypedTable;
}

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class SUntypedTableTreeView : public STableTreeView
{
public:
	UE_API SUntypedTableTreeView();
	UE_API virtual ~SUntypedTableTreeView();

	SLATE_BEGIN_ARGS(SUntypedTableTreeView)
		: _RunInAsyncMode(false)
		{}
		SLATE_ARGUMENT(bool, RunInAsyncMode)
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 * @param InArgs - The declaration data for this widget
	 */
	UE_API void Construct(const FArguments& InArgs, TSharedPtr<FUntypedTable> InTablePtr);

	TSharedPtr<FUntypedTable> GetUntypedTable() const { return StaticCastSharedPtr<FUntypedTable>(GetTable()); }

	UE_API void UpdateSourceTable(TSharedPtr<TraceServices::IUntypedTable> SourceTable);

	UE_API virtual void Reset();

	//////////////////////////////////////////////////
	// IAsyncOperationStatusProvider

	UE_API virtual bool IsRunning() const override;
	UE_API virtual double GetAllOperationsDuration() override;
	virtual double GetCurrentOperationDuration() override { return 0.0; }
	virtual uint32 GetOperationCount() const override { return 1; }
	UE_API virtual FText GetCurrentOperationName() const override;

	//////////////////////////////////////////////////

	UE_API void SetCurrentOperationNameOverride(const FText& InOperationName);
	UE_API void ClearCurrentOperationNameOverride();

	/**
	 * Rebuilds the tree (if necessary).
	 * @param bResync - If true, it forces a resync even if the list did not changed since last sync.
	 */
	UE_API virtual void RebuildTree(bool bResync);

private:
	FStopwatch CurrentOperationStopwatch;
	FText CurrentOperationNameOverride;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef UE_API
