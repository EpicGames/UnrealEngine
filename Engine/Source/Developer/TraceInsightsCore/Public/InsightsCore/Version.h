// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// See https://semver.org/

#define UNREAL_INSIGHTS_VERSION_MAJOR 1
#define UNREAL_INSIGHTS_VERSION_MINOR 9
#define UNREAL_INSIGHTS_VERSION_PATCH 0

#define UNREAL_INSIGHTS_VERSION_STRING "1.09"

/**
 * Extra identifier string (like: "alpha", "beta.1", "beta.2", "test", "featureX", etc.).
 * It can be defined externally, before including this header file.
 */
#ifndef UNREAL_INSIGHTS_VERSION_ID_STRING
//#define UNREAL_INSIGHTS_VERSION_ID_STRING "ue5-main"
#endif

#ifdef UNREAL_INSIGHTS_VERSION_ID_STRING
#define UNREAL_INSIGHTS_VERSION_STRING_EX "v" UNREAL_INSIGHTS_VERSION_STRING "-" UNREAL_INSIGHTS_VERSION_ID_STRING
#else
#define UNREAL_INSIGHTS_VERSION_STRING_EX "v" UNREAL_INSIGHTS_VERSION_STRING
#endif
