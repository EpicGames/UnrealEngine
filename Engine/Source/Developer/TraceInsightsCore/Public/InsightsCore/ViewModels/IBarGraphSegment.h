// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Templates/SharedPointer.h"

namespace UE::Insights
{

class IBarGraphSegment
{
public:
	virtual ~IBarGraphSegment() = default;

	/**
	 * Returns the relative segment size, in interval [0 .. 1].
	 */
	virtual double GetSize() const = 0;

	/**
	 * Returns the text displayed on the segment.
	 */
	virtual FText GetText() const = 0;

	/**
	 * Returns the tool tip text of the segment.
	 */
	virtual FText GetToolTipText() const = 0;

	/**
	 * Returns the color of the segment.
	 */
	virtual FLinearColor GetColor() const = 0;

	/**
	 * Returns the color of the text displayed on the segment.
	 */
	virtual FLinearColor GetTextColor() const = 0;
};

} // namespace UE::Insights
