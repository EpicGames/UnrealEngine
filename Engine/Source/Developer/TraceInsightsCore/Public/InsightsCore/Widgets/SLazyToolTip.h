// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SToolTip.h"

class ILazyToolTipCreator
{
public:
	virtual TSharedPtr<SToolTip> CreateTooltip() const = 0;
};

class SLazyToolTip : public SCompoundWidget, public IToolTip
{
	SLATE_BEGIN_ARGS(SLazyToolTip) {}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, TSharedPtr<const ILazyToolTipCreator> InCreator)
	{
		WeakCreator = InCreator.ToWeakPtr();
	}

	virtual TSharedRef<SWidget> AsWidget()
	{
		CreateToolTipWidget();
		return ToolTipWidget.ToSharedRef();
	}

	virtual TSharedRef<SWidget> GetContentWidget()
	{
		CreateToolTipWidget();
		return ToolTipWidget->GetContentWidget();
	}

	virtual void SetContentWidget(const TSharedRef<SWidget>& InContentWidget)
	{
		CreateToolTipWidget();
		ToolTipWidget->SetContentWidget(InContentWidget);
	}

	void InvalidateWidget()
	{
		ToolTipWidget.Reset();
	}

	virtual bool IsEmpty() const { return false; }
	virtual bool IsInteractive() const { return false; }
	virtual void OnOpening() {}
	virtual void OnClosed() {}

private:
	void CreateToolTipWidget()
	{
		if (!ToolTipWidget.IsValid())
		{
			TSharedPtr<const ILazyToolTipCreator> Creator = WeakCreator.Pin();
			if (Creator.IsValid())
			{
				ToolTipWidget = Creator->CreateTooltip();
			}
			else
			{
				ToolTipWidget = SNew(SToolTip);
			}
			check(ToolTipWidget.IsValid());
		}
	}

private:
	TSharedPtr<SToolTip> ToolTipWidget;
	TWeakPtr<const ILazyToolTipCreator> WeakCreator;
};
