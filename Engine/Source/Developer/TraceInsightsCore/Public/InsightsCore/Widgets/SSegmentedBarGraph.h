// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/IItemsSource.h"

// InsightsCore
#include "InsightsCore/ViewModels/IBarGraphSegment.h"

#define UE_API TRACEINSIGHTSCORE_API

namespace UE::Insights
{

class SSegmentedBarGraph;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FObservableSegmentArrayPointer : public UE::Slate::ItemsSource::IItemsSource<TSharedPtr<IBarGraphSegment>>
{
public:
	using WidgetType = SSegmentedBarGraph;
	using SegmentType = TSharedPtr<IBarGraphSegment>;

	explicit FObservableSegmentArrayPointer(TSharedRef<WidgetType> InGraphWidget, UE::Slate::Containers::TObservableArray<SegmentType>* InSegmentsSource)
		: SegmentsSource(InSegmentsSource)
		, GraphWidgetOwner(InGraphWidget.ToWeakPtr())
	{
		ArrayChangedHandle = InSegmentsSource->OnArrayChanged().AddRaw(this, &FObservableSegmentArrayPointer::HandleArrayChanged);
	}

	virtual ~FObservableSegmentArrayPointer()
	{
		// See ~FObservableArrayPointer() in Widgets\Views\IItemsSource.h.
		checkf(GraphWidgetOwner.IsValid(), TEXT("The widget has a source needed to be released to prevent bad memory access."));
		SegmentsSource->OnArrayChanged().Remove(ArrayChangedHandle);
	}

	virtual const TArrayView<const SegmentType> GetItems() const override
	{
		return TArrayView<const SegmentType>(SegmentsSource->GetData(), SegmentsSource->Num());
	}

	virtual bool IsSame(const void* RawPointer) const override
	{
		return RawPointer == reinterpret_cast<const void*>(&SegmentsSource);
	}

private:
	void HandleArrayChanged(typename UE::Slate::Containers::TObservableArray<SegmentType>::ObservableArrayChangedArgsType Args);

private:
	UE::Slate::Containers::TObservableArray<SegmentType>* SegmentsSource;
	TWeakPtr<WidgetType> GraphWidgetOwner;
	FDelegateHandle ArrayChangedHandle;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FSharedObservableSegmentArray : public UE::Slate::ItemsSource::IItemsSource<TSharedPtr<IBarGraphSegment>>
{
public:
	using WidgetType = SSegmentedBarGraph;
	using SegmentType = TSharedPtr<IBarGraphSegment>;

	explicit FSharedObservableSegmentArray(TSharedRef<WidgetType> InGraphWidget, TSharedRef<UE::Slate::Containers::TObservableArray<SegmentType>> InSegmentsSource)
		: SegmentsSource(InSegmentsSource)
		, GraphWidgetOwner(InGraphWidget.ToWeakPtr())
	{
		ArrayChangedHandle = InSegmentsSource->OnArrayChanged().AddRaw(this, &FSharedObservableSegmentArray::HandleArrayChanged);
	}

	virtual ~FSharedObservableSegmentArray()
	{
		SegmentsSource->OnArrayChanged().Remove(ArrayChangedHandle);
	}

	virtual const TArrayView<const SegmentType> GetItems() const override
	{
		return TArrayView<const SegmentType>(SegmentsSource->GetData(), SegmentsSource->Num());
	}

	virtual bool IsSame(const void* RawPointer) const override
	{
		UE::Slate::Containers::TObservableArray<SegmentType>* ValueToTest = &(SegmentsSource.Get());
		return RawPointer == reinterpret_cast<const void*>(ValueToTest);
	}

private:
	void HandleArrayChanged(typename UE::Slate::Containers::TObservableArray<SegmentType>::ObservableArrayChangedArgsType Args);

private:
	TSharedRef<UE::Slate::Containers::TObservableArray<SegmentType>> SegmentsSource;
	TWeakPtr<WidgetType> GraphWidgetOwner;
	FDelegateHandle ArrayChangedHandle;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * A widget that displays a segmented bar graph.
 */
class SSegmentedBarGraph : public SCompoundWidget
{
public:
	using SegmentType = TSharedPtr<IBarGraphSegment>;
	using SegmentSourceType = UE::Slate::ItemsSource::IItemsSource<SegmentType>;

	/** Default constructor. */
	UE_API SSegmentedBarGraph();

	/** Virtual destructor. */
	UE_API virtual ~SSegmentedBarGraph();

	SLATE_BEGIN_ARGS(SSegmentedBarGraph) {}
		SLATE_ITEMS_SOURCE_ARGUMENT(SegmentType, SegmentsSource)
		TUniquePtr<SegmentSourceType> MakeSegmentsSource(TSharedRef<SSegmentedBarGraph> InWidget) const
		{
			if (_SegmentsSource_ArrayPointer)
			{
				return MakeUnique<UE::Slate::ItemsSource::FArrayPointer<SegmentType>>(_SegmentsSource_ArrayPointer);
			}
			else if (_SegmentsSource_ObservableArrayPointer)
			{
				return MakeUnique<FObservableSegmentArrayPointer>(InWidget, _SegmentsSource_ObservableArrayPointer);
			}
			else if (_SegmentsSource_SharedObservableArray)
			{
				return MakeUnique<FSharedObservableSegmentArray>(InWidget, _SegmentsSource_SharedObservableArray.ToSharedRef());
			}
			return TUniquePtr<SegmentSourceType>();
		}
	SLATE_END_ARGS()

	/**
	 * Construct this widget.
	 * @param InArgs - The declaration data for this widget
	 */
	UE_API void Construct(const FArguments& InArgs);

	UE_API void RequestGraphRefresh();

	/**
	 * Set the SegmentsSource. The graph bar will generate widgets to represent these segments.
	 * @param InSegmentsSource A pointer to the array of segments that should be observed by this widget.
	*/
	void SetSegmentsSource(const TArray<SegmentType>* InSegmentsSource)
	{
		ensureMsgf(InSegmentsSource, TEXT("The SegmentsSource is invalid."));
		if (SegmentsSource == nullptr || !SegmentsSource->IsSame(reinterpret_cast<const void*>(InSegmentsSource)))
		{
			if (InSegmentsSource)
			{
				SetSegmentsSource(MakeUnique<UE::Slate::ItemsSource::FArrayPointer<SegmentType>>(InSegmentsSource));
			}
			else
			{
				ClearSegmentsSource();
			}
		}
	}

	void SetSegmentsSource(TSharedRef<UE::Slate::Containers::TObservableArray<SegmentType>> InSegmentsSource)
	{
		if (SegmentsSource == nullptr || !SegmentsSource->IsSame(reinterpret_cast<const void*>(&InSegmentsSource.Get())))
		{
			SetSegmentsSource(MakeUnique<FSharedObservableSegmentArray>(this->SharedThis(this), MoveTemp(InSegmentsSource)));
		}
	}

	void SetSegmentsSource(TUniquePtr<SegmentSourceType> Provider)
	{
		SegmentsSource = MoveTemp(Provider);
		RequestGraphRefresh();
	}

	void ClearSegmentsSource()
	{
		SetSegmentsSource(TUniquePtr<SegmentSourceType>());
	}

	bool HasValidSegmentsSource() const
	{
		return SegmentsSource != nullptr;
	}

	TArrayView<const SegmentType> GetSegments() const
	{
		return SegmentsSource ? SegmentsSource->GetItems() : TArrayView<const SegmentType>();
	}

private:
	void AddHorizontalSegment(TSharedRef<SHorizontalBox> Box, int32 SegmentIndex);
	float GetSegmentSize(int32 SegmentIndex) const;
	FText GetSegmentText(int32 SegmentIndex) const;
	FText GetSegmentToolTipText(int32 SegmentIndex) const;
	FSlateColor GetSegmentColor(int32 SegmentIndex) const;
	FSlateColor GetSegmentTextColor(int32 SegmentIndex) const;

private:
	/** Pointer to the source data that we are observing */
	TUniquePtr<SegmentSourceType> SegmentsSource;

	TSharedPtr<SHorizontalBox> HorizontalBox;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef UE_API
