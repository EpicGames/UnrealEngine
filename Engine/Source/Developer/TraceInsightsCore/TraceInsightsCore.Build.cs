// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TraceInsightsCore : ModuleRules
{
	public TraceInsightsCore(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"ApplicationCore",
				"Core",
				"CoreUObject",
				"DesktopPlatform",
				"InputCore",
				"Slate",
				"SlateCore",
				"ToolWidgets",
				"TraceServices",
			}
		);
	}
}
